[/Script/FastGeoStreaming.FastGeoWorldPartitionRuntimeCellTransformer]
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.PackedLevelActor'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.StaticMeshActor'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.WorldPartitionHLOD'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.WorldPartitionAutoInstancedActor'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.PointLight'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.SpotLight'
+BuiltinAllowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.RectLight'
+BuiltinDisallowedActorClasses=/Script/CoreUObject.Class'/Script/Water.WaterBody'
+BuiltinDisallowedActorClasses=/Script/CoreUObject.Class'/Script/Engine.Info'
+BuiltinAllowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.StaticMeshComponent'
+BuiltinAllowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.DecalComponent'
+BuiltinAllowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.PointLightComponent'
+BuiltinAllowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.SpotLightComponent'
+BuiltinAllowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.RectLightComponent'
+BuiltinDisallowedComponentClasses=/Script/CoreUObject.Class'/Script/Foliage.InteractiveFoliageComponent'
+BuiltinDisallowedComponentClasses=/Script/CoreUObject.Class'/Script/Landscape.LandscapeNaniteComponent'
+BuiltinDisallowedComponentClasses=/Script/CoreUObject.Class'/Script/Landscape.LandscapeMeshProxyComponent'
+BuiltinDisallowedComponentClasses=/Script/CoreUObject.Class'/Script/Water.WaterBodyMeshComponent'
+BuiltinDisallowedComponentClasses=/Script/CoreUObject.Class'/Script/Engine.SplineMeshComponent'
+BuiltinIgnoredRemainingComponentClasses=/Script/CoreUObject.Class'/Script/Engine.ActorTextureStreamingBuildDataComponent'

