{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "FastGeo Streaming",
	"Description": "A system that extracts and converts a partitioned world's geometry to optimize world streaming performance.",
	"Category": "World Building",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "FastGeoStreaming",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "FastGeoStreamingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}