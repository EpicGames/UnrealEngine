// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FastGeoStreaming : ModuleRules
	{
		public FastGeoStreaming(ReadOnlyTargetRules Target) : base(Target)
		{
			// FastGeo's render-state, scene-proxy, and physics integrations consume engine
			// headers (e.g. SimpleStreamableAssetManager internals, async physics state hooks)
			// that live under Engine/Internal and have not been promoted to the public Engine
			// module surface. The dependency is intentional and constrained to this private
			// include path; promote individual headers to Engine/Public when their API
			// stabilizes and remove this entry.
			PrivateIncludePaths.Add(System.IO.Path.Combine(EngineDirectory, "Source/Runtime/Engine/Internal"));

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"NavigationSystem",
					"Chaos",
					"PhysicsCore",
					"RenderCore",
					"RHI",
					"TraceLog"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}