// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Tasks/Pipe.h"
#include "FastGeoContainer.h"

class FSceneInterface;

struct FFastGeoAsyncRenderStateJobQueue
{
	FFastGeoAsyncRenderStateJobQueue(FSceneInterface* Scene);
	~FFastGeoAsyncRenderStateJobQueue();

	enum class EJobType
	{
		PrecachePSOs,
		CreateRenderState,
		DestroyRenderState,
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		DeferredCreateRenderState,
		RecreateRenderState,
#endif
	};

	struct FJob
	{
		FJob(UFastGeoContainer* InFastGeo, EJobType InType)
			: FastGeo(InFastGeo)
			, Type(InType)
		{}

		bool IsValid() const { return ::IsValid(FastGeo); }
		void Execute() const;
		void OnPostExecute_GameThread() const;

		inline bool operator == (const FJob& Other) const
		{
			return (FastGeo == Other.FastGeo) && (Type == Other.Type);
		}

		friend inline uint32 GetTypeHash(const FJob& InJob)
		{
			return GetTypeHash(TPair<UFastGeoContainer*, EJobType>{InJob.FastGeo, InJob.Type});
		}

		UFastGeoContainer* FastGeo;
		EJobType Type;
	};

	struct FJobSet
	{
		void Add(const FJob& Job)
		{
			Jobs.Add(Job);
		}

		int32 Remove(const FJob& Job)
		{
			return Jobs.Remove(Job);
		}

		bool IsEmpty() const
		{
			return Jobs.IsEmpty();
		}

		void Execute() const;
		void OnPostExecute_GameThread() const;

	private:
		TSet<FJob> Jobs;
	};

	void Tick(bool bForceWaitCompletion);
	void AddJob(const FJob& Job);
	bool IsCompleted() const;

private:

	void Launch();
	void WaitForAsyncTasksExecution();
	bool AreAsyncTasksExecuted() const;
	void OnAsyncTasksExecuted();
	void OnUpdateLevelStreamingDone();
	void TriggerIsReadyToRunAsyncTasksEvent();

	FSceneInterface* Scene;
	FDelegateHandle OnAllLevelsChangedHandle;
	using FJobs = TUniquePtr<FJobSet>;
	FJobs PendingJobs;
	// PipedJobs and PipeTasks are parallel arrays kept aligned by index: PipeTasks[i] is the
	// pipe task that executes PipedJobs[i]. Both are appended to in Launch() and removed at
	// matching indices in Tick() and OnAsyncTasksExecuted(). All mutations must preserve the
	// alignment invariant (PipedJobs.Num() == PipeTasks.Num()).
	TArray<FJobs> PipedJobs;
	TArray<UE::Tasks::FTask> PipeTasks;
	TOptional<UE::Tasks::FTaskEvent> IsReadyToRunAsyncTasksEvent;
	UE::Tasks::FPipe Pipe{ TEXT("FastGeoAsyncJobQueue") };
};