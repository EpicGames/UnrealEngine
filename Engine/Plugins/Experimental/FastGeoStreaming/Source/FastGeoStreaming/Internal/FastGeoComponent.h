// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PSOPrecacheFwd.h"
#include "PSOPrecacheComponent.h"
#include "IFastGeoElement.h"
#include "FastGeoElementType.h"
#include "Physics/Experimental/AsyncPhysicsStateProcessorInterface.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Components/ComponentInterfaces.h"
#include "Components/SceneComponent.h"
#include "Async/TaskGraphInterfaces.h"

class UWorld;
class UBodySetup;
class UActorComponent;
class FFastGeoComponentCluster;
class FFastGeoDestroyRenderStateContext;
class FWeakFastGeoComponent;
class UFastGeoContainer;
class FSceneInterface;
class FPrimitiveSceneProxy;
class FRegisterComponentContext;
struct FBodyInstance;

// Uncomment to debug FastGeo physics state creation/destruction
// #define FASTGEO_DEBUG_PHYSICS

enum class ESceneProxyCreationError
{
	None,
	WaitingPSOs,
	InvalidMesh
};

class FASTGEOSTREAMING_API FFastGeoComponent : public IFastGeoElement
{
public:
	typedef IFastGeoElement Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	FFastGeoComponent(const FFastGeoComponent& Other);
	virtual ~FFastGeoComponent() = default;

#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component);
#endif
	virtual void Serialize(FArchive& Ar);
	virtual void InitializeDynamicProperties() {}
	virtual void ApplyWorldTransform(const FTransform& InTransform);
	UWorld* GetWorld() const;
	FSceneInterface* GetScene() const;
	FPhysScene* GetPhysicsScene() const;
	float GetWorldTimeSeconds() const;
	UFastGeoContainer* GetOwnerContainer() const;
	FFastGeoComponentCluster* GetOwnerComponentCluster() const;
	int32 GetComponentIndex() const { return ComponentIndex; }
	FLinearColor GetDebugColor() const;

	// Enumerates all materials this component references, including Nanite overrides.
	// bIsNaniteOverride is true for materials obtained via GetNaniteOverride().
	virtual void ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const {}

	// Render interface
	virtual void UpdateVisibility() {}
	virtual bool ShouldComponentAddToRenderScene() const;
	virtual void CreateRenderState(FRegisterComponentContext* Context) {}
	virtual void DestroyRenderState(FFastGeoDestroyRenderStateContext* Context) {}
	void MarkRenderStateDirty(bool bEvenIfNotCreated);
	bool IsRenderStateDirty() const;
	bool IsRenderStateDelayed() const;
	bool IsRenderStateCreated() const;

	// Physics interface
	virtual bool IsCollisionEnabled() const { return false; }
	virtual UBodySetup* GetBodySetup() const { return nullptr; }

	// Async physics state creation/destruction
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread() {}
	virtual void OnAsyncCreatePhysicsState();
	virtual void OnAsyncCreatePhysicsStateEnd_GameThread();
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread();
	virtual void OnAsyncDestroyPhysicsState();
	virtual void OnAsyncDestroyPhysicsStateEnd_GameThread();
	virtual void GetBodyInstances(TArray<FBodyInstance*>& OutBodyInstances) {}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// PSO Precaching interface
	virtual bool ShouldPrecachePSOs() const;
	virtual void PrecachePSOs_Concurrent() {}
	virtual FPSOPrecacheComponentData& GetPSOPrecacheComponentData() = 0;
	virtual const FPSOPrecacheComponentData& GetPSOPrecacheComponentData() const = 0;
	// Returns true if the proxy was created with a fallback material and needs recreation when PSO completes.
	virtual bool NeedsPSORecreate() const { return false; }
#endif

protected:
	// Persistent Data
	int32 ComponentIndex = INDEX_NONE;
	FTransform LocalTransform{ FTransform::Identity };
	FTransform WorldTransform{ FTransform::Identity };
	TEnumAsByte<enum EDetailMode> DetailMode = EDetailMode::DM_Low;

	// Transient Data
	FFastGeoComponentCluster* Owner = nullptr;

	// Per-component RWLock guarding the (ProxyState, scene-proxy pointer) pair so readers
	// always see a consistent tuple. Reader: render-thread UpdateVisibility lambda (cluster).
	// Writers: Create/DestroyRenderState (worker in cooked-async, GT in sync) and the cluster
	// UpdateVisibility helpers. Spans Scene->AddPrimitive in Create because the proxy pointer
	// is populated by AddPrimitive and must become visible atomically with ProxyState=Created.
	TUniquePtr<FRWLock> Lock;

#ifdef FASTGEO_DEBUG_PHYSICS
	// Physics state
	enum EPhysicsStateCreation
	{
		NotCreated,
		Creating,
		Created,
		Destroying
	};

	EPhysicsStateCreation PhysicsStateCreation = EPhysicsStateCreation::NotCreated;
#endif

	// Render state
	enum class EProxyCreationState : uint8
	{
		None, // Constructed/Initialized
		Creating, // Actively creating the proxy
		Created, // Proxy is now created
		Delayed // Proxy creation delayed (used when PSO precaching is not ready when creating proxy)
	};

	EProxyCreationState ProxyState = EProxyCreationState::None;
	bool bRenderStateDirty = false;

private:
	void SetOwnerComponentCluster(FFastGeoComponentCluster* InOwner);
	friend class FFastGeoComponentCluster;
	friend FArchive& operator<<(FArchive& Ar, FFastGeoComponent& Component);
};