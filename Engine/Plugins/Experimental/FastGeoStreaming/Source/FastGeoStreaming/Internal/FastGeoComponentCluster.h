// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFastGeoElement.h"
#include "FastGeoElementType.h"
#include "FastGeoStaticMeshComponent.h"
#include "FastGeoInstancedStaticMeshComponent.h"
#include "FastGeoSkinnedMeshComponent.h"
#include "FastGeoInstancedSkinnedMeshComponent.h"
#include "FastGeoProceduralISMComponent.h"
#include "FastGeoDecalComponent.h"
#include "FastGeoRectLightComponent.h"
#include "FastGeoSpotLightComponent.h"
#include "FastGeoPointLightComponent.h"

class ULevel;
class UFastGeoContainer;
class FFastGeoComponent;
class FArchive;

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif // WITH_EDITOR

class FFastGeoComponentCluster : public IFastGeoElement
{
public:
	typedef IFastGeoElement Super;

	/** Static type identifier for this element class */
	FASTGEOSTREAMING_API static const FFastGeoElementType Type;

	FASTGEOSTREAMING_API FFastGeoComponentCluster(UFastGeoContainer* InOwner = nullptr, FName InName = NAME_None, FFastGeoElementType InType = Type);

	FFastGeoComponentCluster(const FFastGeoComponentCluster& InOther);

	virtual ~FFastGeoComponentCluster() = default;
	virtual void OnRegister() {}
	virtual void OnUnregister() {}
	virtual void Serialize(FArchive& Ar);
	virtual bool IsVisible() const { return true; }

	FASTGEOSTREAMING_API UFastGeoContainer* GetOwnerContainer() const;
	ULevel* GetLevel() const;
	FASTGEOSTREAMING_API FFastGeoComponent* GetComponent(uint32 InComponentTypeID, int32 InComponentIndex);
	const FString& GetName() const { return Name; }
	int32 GetComponentClusterIndex() const { return ComponentClusterIndex; }
	void UpdateVisibility();

	FASTGEOSTREAMING_API bool HasComponents() const;
	virtual FFastGeoComponent& AddComponent(FFastGeoElementType InComponentType);

#if WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) {}
	void SerializeWithComponentFilter(FArchive& Ar, TFunctionRef<bool(const FFastGeoComponent&)> ShouldSerializeComponent);
#endif

	template <typename TComponent = const FFastGeoComponent, typename TFunc>
	void ForEachComponent(TFunc&& InFunc) const
	{
		ForEachComponent<const FFastGeoComponentCluster, const TComponent, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponent = FFastGeoComponent, typename TFunc>
	void ForEachComponent(TFunc&& InFunc)
	{
		ForEachComponent<FFastGeoComponentCluster, TComponent, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponent = const FFastGeoComponent, typename TFunc>
	bool ForEachComponentBreakable(TFunc&& InFunc) const
	{
		return ForEachComponentBreakable<const FFastGeoComponentCluster, const TComponent, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponent = FFastGeoComponent, typename TFunc>
	bool ForEachComponentBreakable(TFunc&& InFunc)
	{
		return ForEachComponentBreakable<FFastGeoComponentCluster, TComponent, TFunc>(this, Forward<TFunc>(InFunc));
	}

protected:
	// Transient Data
	UFastGeoContainer* Owner;

	// Persistent Data
	FString Name;
	int32 ComponentClusterIndex;

	TArray<FFastGeoStaticMeshComponent> StaticMeshComponents;
	TArray<FFastGeoInstancedStaticMeshComponent> InstancedStaticMeshComponents;
	TArray<FFastGeoSkinnedMeshComponent> SkinnedMeshComponents;
	TArray<FFastGeoInstancedSkinnedMeshComponent> InstancedSkinnedMeshComponents;
	TArray<FFastGeoProceduralISMComponent> ProceduralISMComponents;
	TArray<FFastGeoDecalComponent> DecalComponents;
	TArray<FFastGeoRectLightComponent> RectLightComponents;
	TArray<FFastGeoSpotLightComponent> SpotLightComponents;
	TArray<FFastGeoPointLightComponent> PointLightComponents;

	// Iterates all component arrays. Lambda can return void (non-breakable) or bool (return false to break).
	template <typename Func>
	void ForEachComponentArray(Func&& InFunc)
	{
		auto Call = [&InFunc](auto& Array) -> bool
		{
			if constexpr (std::is_same_v<decltype(InFunc(Array)), bool>)
			{
				return InFunc(Array);
			}
			else
			{
				InFunc(Array);
				return true;
			}
		};

		Call(StaticMeshComponents) &&
		Call(InstancedStaticMeshComponents) &&
		Call(SkinnedMeshComponents) &&
		Call(InstancedSkinnedMeshComponents) &&
		Call(ProceduralISMComponents) &&
		Call(DecalComponents) &&
		Call(RectLightComponents) &&
		Call(SpotLightComponents) &&
		Call(PointLightComponents);
	}

	template <typename Func>
	void ForEachComponentArray(Func&& InFunc) const
	{
		const_cast<FFastGeoComponentCluster*>(this)->ForEachComponentArray([&InFunc](auto& Array) -> decltype(InFunc(std::as_const(Array)))
		{
			return InFunc(std::as_const(Array));
		});
	}

	template <typename ThisType, typename TComponent, typename TFunc>
	static void ForEachComponent(ThisType* Self, TFunc&& InFunc)
	{
		Self->ForEachComponentArray([&InFunc](auto& InArray)
		{
			typedef typename TDecay<decltype(InArray)>::Type ArrayType;
			using TArrayComponent = typename ArrayType::ElementType;

			if constexpr (std::is_same<TComponent, FFastGeoComponent>::value)
			{
				for (auto& Component : InArray)
				{
					InFunc(Component);
				}
			}
			else if (TArrayComponent::Type.IsA(TComponent::Type))
			{
				for (auto& Component : InArray)
				{
					checkSlow(Component.template IsA<TComponent>());
					Forward<TFunc>(InFunc)(*(TComponent*)(&Component));
				}
			}
		});
	}

	template <typename ThisType, typename TComponent, typename TFunc>
	static bool ForEachComponentBreakable(ThisType* Self, TFunc&& InFunc)
	{
		bool bContinue = true;
		Self->ForEachComponentArray([&InFunc, &bContinue](auto& InArray) -> bool
		{
			typedef typename TDecay<decltype(InArray)>::Type ArrayType;
			using TArrayComponent = typename ArrayType::ElementType;

			if constexpr (std::is_same<TComponent, FFastGeoComponent>::value)
			{
				for (auto& Component : InArray)
				{
					if (!InFunc(Component))
					{
						bContinue = false;
						return false;
					}
				}
			}
			else if (TArrayComponent::Type.IsA(TComponent::Type))
			{
				for (auto& Component : InArray)
				{
					checkSlow(Component.template IsA<TComponent>());
					if (!Forward<TFunc>(InFunc)(*(TComponent*)(&Component)))
					{
						bContinue = false;
						return false;
					}
				}
			}
			return true;
		});
		return bContinue;
	}

private:

#if !WITH_EDITOR
	int32 StripRenderOnlyComponents();
#endif
	void ForceUpdateVisibility(const TArray<FFastGeoPrimitiveComponent*>& Components, int32 UpdateCounter);
	void ForceUpdateVisibilityOrDeferTask(const TArray<FFastGeoPrimitiveComponent*>& Components, int32 UpdateCounter);
	void UpdateVisibility_Internal(TArray<FFastGeoPrimitiveComponent*>&& ShowComponents, TArray<FFastGeoPrimitiveComponent*>&& HideComponents, int32 UpdateCounter = 0);
	void InitializeDynamicProperties();
	void SetOwnerContainer(UFastGeoContainer* InOwner);
	void SetComponentClusterIndex(int32 InComponentClusterIndex);
	friend class UFastGeoContainer;

	friend FArchive& operator<<(FArchive& Ar, FFastGeoComponentCluster& ComponentCluster);
};
