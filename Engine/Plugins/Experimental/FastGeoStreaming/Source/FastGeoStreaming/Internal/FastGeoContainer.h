// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "Physics/Experimental/AsyncPhysicsStateProcessorInterface.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "AI/Navigation/NavigationElement.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoHLOD.h"
#include "Containers/BitArray.h"
#include "FastGeoRenderStateUtils.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif // WITH_EDITOR

#include "FastGeoContainer.generated.h"

class UWorld;
class ULevel;
class URuntimeVirtualTexture;
class UStaticMesh;
class UMaterialInterface;
class UNavigationSystemBase;
class UFastGeoWorldSubsystem;
class FFastGeoComponent;
class FFastGeoComponentCluster;
class FChaosUserDefinedEntity;
class AFastGeoSurrogateActor;

/** Result of UFastGeoContainer::CreateRuntime. */
struct FFastGeoCreateRuntimeResult
{
	/** The created container, or null on failure. */
	UFastGeoContainer* Container = nullptr;

	/** Component pointers built from the container's internal component iteration order. */
	TArray<FFastGeoComponent*> Components;
};

/**
 * Cached world/scene/subsystem/physics pointers.
 * Populated on GT before workers run.
 * Workers read from this instead of performing lookups not safe from generic workers
 * (e.g., subsystem collection access, world time reads, physics scene access).
 * Reset on unregister. Re-populated on re-register.
 * Lifetime: World/Scene/Subsystem all outlive the container's registered state.
 */
struct FFastGeoAsyncContext
{
	void Initialize(UFastGeoContainer& Container);
	void Reset()
	{
		bInitialized = false;
		World = nullptr;
		Scene = nullptr;
		PhysicsScene = nullptr;
		WorldSubsystem = nullptr;
		TimeSeconds.store(0.0f, std::memory_order_relaxed);
	}
	bool IsInitialized() const { return bInitialized; }

	UWorld* GetWorld() const { check(bInitialized); return World; }
	FSceneInterface* GetScene() const { check(bInitialized); return Scene; }
	FPhysScene* GetPhysicsScene() const { check(bInitialized); return PhysicsScene; }
	UFastGeoWorldSubsystem* GetWorldSubsystem() const { check(bInitialized); return WorldSubsystem; }
	float GetTimeSeconds() const { check(bInitialized); return TimeSeconds.load(std::memory_order_relaxed); }
	void UpdateTimeSeconds(float InTime) { check(bInitialized); TimeSeconds.store(InTime, std::memory_order_relaxed); }

private:
	bool bInitialized = false;
	UWorld* World = nullptr;
	FSceneInterface* Scene = nullptr;
	FPhysScene* PhysicsScene = nullptr;
	UFastGeoWorldSubsystem* WorldSubsystem = nullptr;
	std::atomic<float> TimeSeconds = 0.0f;
};

/**
 * Calls ConditionalPostLoad on all materials referenced by the container.
 * Ensures RF_NeedPostLoad is cleared before workers access materials.
 * Permanent - never reset. RF_NeedPostLoad is cleared once.
 */
struct FFastGeoMaterialPostLoadInit
{
	void Initialize(UFastGeoContainer& Container);
	bool IsInitialized() const { return bInitialized; }

private:
	bool bInitialized = false;
};

/**
 * Warms UMaterialInterface::CachedRelevance for all materials + default material.
 * Workers use the cached relevance instead of reading GameThreadShaderMap.
 * Permanent - never reset. Cache persists on UMaterialInterface.
 * Applies to async mode only.
 */
struct FFastGeoMaterialRelevanceInit
{
	void Initialize(UFastGeoContainer& Container);
	bool IsInitialized() const { return bInitialized; }

private:
	bool bInitialized = false;
};

struct FFastGeoComponentClusterPendingState
{
	FFastGeoRenderStateBatch RenderState;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	FFastGeoRenderStateBatch PrecachePSOState;
#endif

	struct FPhysicsState
	{
		FPhysicsState()
		{
			Reset();
		}

		void Reset()
		{
			ComponentsToProcess = nullptr;
			TotalNumProcessed = 0;
		}

		bool IsCompleted() const
		{
			return !ComponentsToProcess || (TotalNumProcessed >= ComponentsToProcess->Num());
		}

		TArray<FFastGeoComponent*>* ComponentsToProcess;
		std::atomic<int32> TotalNumProcessed;
	} PhysicsState;

	FFastGeoComponentClusterPendingState()
	{
		Reset();
	}

	void Reset()
	{
		RenderState.Reset();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		PrecachePSOState.Reset();
#endif
		PhysicsState.Reset();
	}

	bool HasAnyPendingState(bool bIncludePSOPrecaching = true) const
	{
		bool bHasAnyPendingState = !RenderState.IsCompleted() || !PhysicsState.IsCompleted();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		if (bIncludePSOPrecaching)
		{
			bHasAnyPendingState |= !PrecachePSOState.IsCompleted();
		}
#endif
		return bHasAnyPendingState;
	}
};

UCLASS(Within = Level)
class FASTGEOSTREAMING_API UFastGeoContainer : public UAssetUserData, public IAsyncPhysicsStateProcessor, public IPhysicsBodyInstanceOwnerResolver
{
	GENERATED_BODY()

public:

	/**
	 * Creates a runtime FastGeoContainer and starts async registration.
	 * The subsystem drives registration ticks automatically.
	 * Subscribe to GetOnRegistered() for completion notification, or check IsFullyRegistered().
	 *
	 * @param InWorld				World in which to create the container.
	 * @param InDebugName			Debug name for the container and its component cluster.
	 * @param InInitCluster			Callback to populate the component cluster with components.
	 * @param bInCollectReferences	Whether to collect asset references via serialization archive. Pass false if the caller manages asset lifetimes externally.
	 *								
	 * @return Result containing the container and an ordered array of component pointers matching the components added in InInitCluster. Container is null on failure.
	 */
	static FFastGeoCreateRuntimeResult CreateRuntime(UWorld* InWorld, FName InDebugName, TFunctionRef<void(FFastGeoComponentCluster&)> InInitCluster, bool bInCollectReferences = true);

	/**
	 * Destroys a runtime FastGeoContainer created by CreateRuntime.
	 * Initiates async unregistration; the subsystem drives completion.
	 *
	 * @param InFastGeo		The runtime container to destroy. Must satisfy IsRuntime().
	 */
	static void DestroyRuntime(UFastGeoContainer* InFastGeo);

	/** Returns true if this is a runtime (transient) container. Only CreateRuntime sets RF_Transient on FastGeo containers. */
	bool IsRuntime() const { return HasAnyFlags(RF_Transient); }

	void Register();
	void Unregister();
	bool IsRegistering() const;
	bool IsRegistered() const;
	bool IsUnregistering() const;
	bool IsUnregistered() const;
	bool IsFullyRegistered() const;
	bool IsFullyUnregistered() const;
	void Tick(bool bWaitForCompletion = false);
	bool HasAnyPendingTasks(bool bIncludePSOPrecaching = true) const;
	// Returns true if registration-related tasks (create/destroy) are pending.
	// Excludes post-registration PSO recreate work (UseFallbackMaterialUntilPSOPrecached).
	bool HasAnyPendingRegistrationTasks(bool bIncludePSOPrecaching = true) const;
	bool HasAnyPendingCreateTasks(bool bIncludePSOPrecaching = true) const;
	bool HasAnyPendingDestroyTasks() const;
	void PrecachePSOs();
	UWorld* GetWorld() const;
	ULevel* GetLevel() const;
	FFastGeoComponentCluster* GetComponentCluster(uint32 InComponentClusterTypeID, int32 InComponentClusterIndex);

	TArray<IPrimitiveComponent*> GetPrimitiveComponents() const;
	TArray<IStaticMeshComponent*> GetStaticMeshComponents() const;

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
	//~ End UObject Interface

	//~Begin IPhysicsBodyInstanceOwnerResolver
	virtual IPhysicsBodyInstanceOwner* ResolvePhysicsBodyInstanceOwner(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	//~End IPhysicsBodyInstanceOwnerResolver

	bool IsUsingSurrogateComponents() const;
	AFastGeoSurrogateActor* GetSurrogateActor() const;

	void AddComponentCluster(FFastGeoComponentCluster* ComponentCluster);

	void InitializeDynamicProperties(bool bInitForPlay = false);

	void OnPrecachePSOsBegin_GameThread();
	void OnPrecachePSOs_Concurrent();
	void OnPrecachePSOsEnd_GameThread();

	void OnCreateRenderStateBegin_GameThread();
	void OnCreateRenderState_Concurrent();
	void OnCreateRenderStateEnd_GameThread();

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Called from PSO precache completion callback. Routes to the appropriate path:
	// - DelayUntilPSOPrecached: moves component to ready set for deferred create.
	// - UseFallbackMaterialUntilPSOPrecached: pushes to recreate queue for material swap.
	// - Otherwise: no-op (early completion detected by CheckPSODeferredReadiness polling).
	void OnComponentPSOPrecacheCompleted(FFastGeoComponent* Component);

	// DelayUntilPSOPrecached: deferred proxy creation for components whose PSOs were not ready.
	void OnPSODeferredReady(FFastGeoComponent* Component);
	void OnDeferredCreateRenderStateBegin_GameThread();
	void OnDeferredCreateRenderState_Concurrent();
	void OnDeferredCreateRenderStateEnd_GameThread();

	// Checks if deferred components are ready for proxy creation and triggers a deferred create pass.
	// bPollPending: when true, polls IsPSOPrecaching() on pending components to detect lost PSO callbacks (safety net).
	// Called with bPollPending=true from:
	// - WorldSubsystem::Tick (periodic safety net poll)
	// - OnCreateRenderStateEnd_GameThread (detect callbacks that fired before parking)
	// - OnDeferredCreateRenderStateEnd_GameThread (re-check after batch completes)
	// Called with bPollPending=false from:
	// - OnPSODeferredReady (per-callback, skips O(n) poll).
	void CheckPSODeferredReadiness(bool bPollPending = false);

	// UseFallbackMaterialUntilPSOPrecached: recreate proxy with real material once PSOs are ready.
	void RoutePSORecreateComponent(FFastGeoComponent* Component);
	void PushPSORecreateComponent(FFastGeoComponent* Component);
	void OnRecreateRenderStateBegin_GameThread();
	void OnRecreateRenderState_Concurrent();
	void OnRecreateRenderStateEnd_GameThread();

	bool HasPSODeferredComponents() const { return !PSODeferredPendingComponents.IsEmpty() || !PSODeferredReadyComponents.IsEmpty(); }
	bool HasAnyPendingRecreateTasks() const;
#endif
	
	void OnDestroyRenderStateBegin_GameThread();
	void OnDestroyRenderState_Concurrent();
	void OnDestroyRenderStateEnd_GameThread();

	void OnCreatePhysicsStateBegin_GameThread();
	void OnDestroyPhysicsStateBegin_GameThread();

	//~ Begin IAsyncPhysicsStateProcessor
	virtual bool AllowsAsyncPhysicsStateCreation() const override final;
	virtual bool AllowsAsyncPhysicsStateDestruction() const override final;
	virtual bool IsAsyncPhysicsStateCreated() const override final;
	virtual UObject* GetAsyncPhysicsStateObject() const override final;
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	virtual bool OnAsyncCreatePhysicsState(const UE::FTimeout& Timeout) override;
	virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override;
	virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
	virtual void CollectBodySetupsWithPhysicsMeshesToCreate(TSet<UBodySetup*>& OutBodySetups) const override;
	//~ End IAsyncPhysicsStateProcessor

	template <typename TComponentCluster = const FFastGeoComponentCluster, typename TFunc>
	void ForEachComponentCluster(TFunc&& InFunc) const
	{
		ForEachComponentCluster<const UFastGeoContainer, const TComponentCluster, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponentCluster = FFastGeoComponentCluster, typename TFunc>
	void ForEachComponentCluster(TFunc&& InFunc)
	{
		ForEachComponentCluster<UFastGeoContainer, TComponentCluster, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponentCluster = const FFastGeoComponentCluster, typename TFunc>
	bool ForEachComponentClusterBreakable(TFunc&& InFunc) const
	{
		return ForEachComponentClusterBreakable<const UFastGeoContainer, const TComponentCluster, TFunc>(this, Forward<TFunc>(InFunc));
	}

	template <typename TComponentCluster = FFastGeoComponentCluster, typename TFunc>
	bool ForEachComponentClusterBreakable(TFunc&& InFunc)
	{
		return ForEachComponentClusterBreakable<UFastGeoContainer, TComponentCluster, TFunc>(this, Forward<TFunc>(InFunc));
	}

	DECLARE_MULTICAST_DELEGATE(FOnRegistered);
	FOnRegistered& GetOnRegistered() { return OnRegistered; }

	DECLARE_MULTICAST_DELEGATE(FOnUnregistered);
	FOnUnregistered& GetOnUnregistered() { return OnUnregistered; }

protected:

	FFastGeoComponentClusterPendingState PendingCreate;
	FFastGeoComponentClusterPendingState PendingDestroy;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// DelayUntilPSOPrecached: components whose proxy creation was deferred because PSOs were not ready.
	// Part of registration -- blocks AddToWorld until all deferred proxies are created.
	// Components start in PSODeferredPendingComponents. As each PSO callback fires, the
	// component moves to PSODeferredReadyComponents. When the pending set is empty (or the
	// safety poll detects all are ready), the ready array becomes ComponentsToProcess for
	// the deferred create pass.
	TSet<FFastGeoComponent*> PSODeferredPendingComponents;
	TArray<FFastGeoComponent*> PSODeferredReadyComponents;
	FFastGeoRenderStateBatch PSODeferredCreateState;
	bool bIsProcessingPSODeferredCreate = false;

	// UseFallbackMaterialUntilPSOPrecached: components created with fallback material that
	// need recreation with the real material once PSOs are ready.
	// Post-registration -- does NOT block AddToWorld.
	// Populated by PushPSORecreateComponent (async-only, asserted), processed by the async
	// job queue via StartRecreateRenderStateWork -> PushAsyncRecreateRenderStateJob.
	// Sync mode does not use these fields -- it routes through MarkRenderStateDirty ->
	// ProcessPendingRecreate instead.
	TArray<FFastGeoComponent*> PSORecreateComponents;
	FFastGeoRenderStateBatch PSORecreateState;
	bool bIsProcessingPSORecreate = false;
#endif

public:

	// Worker-safe accessors that route through the GT-populated FFastGeoAsyncContext when initialized.
	// Component code on async render-state workers must use these instead of GetWorld()->GetSubsystem...,
	// World->Scene, GetWorld()->GetPhysicsScene(), or GetWorld()->GetTimeSeconds() to avoid touching
	// GT-mutable engine state from a generic worker thread.
	UFastGeoWorldSubsystem* GetWorldSubsystem() const;
	FSceneInterface* GetScene() const;
	FPhysScene* GetPhysicsScene() const;
	float GetWorldTimeSeconds() const;

private:

	void OnCreated(bool bCollectReferences = true);
	void InitializePrerequisites();
	void ForEachMaterialAsset(TFunctionRef<void(UMaterialInterface*)> Func);
	void BuildMaterialAssets();
#if !WITH_EDITOR
	void ConditionalStripRenderOnlyComponentsAndAssets();
	void CacheMaterialAssets();
#endif

	void StartCreatePhysicsStateWork();
	void StartDestroyPhysicsStateWork();
	void TickPhysicsWork(bool bWaitForCompletion);

	void StartPrecachePSOWork();
	void StartCreateRenderStateWork();
	void StartDestroyRenderStateWork();
	template <bool bDestroyFirst>
	void ExecuteCreateRenderState_Concurrent(FFastGeoRenderStateBatch& State);
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	void ForceCreateDeferredComponents();
	void StartDeferredCreateRenderStateWork();
	void StartRecreateRenderStateWork();
	bool TickDeferredCreate_Sync(const UE::FTimeout& Timeout);
#endif
	void TickRenderWork(bool bWaitForCompletion);
	void TickRenderWork_Sync(bool bWaitForCompletion);
	bool TickPrecachePSO_Sync(const UE::FTimeout& Timeout);
	bool TickCreateRenderState_Sync(const UE::FTimeout& Timeout);
	bool TickDestroyRenderState_Sync(const UE::FTimeout& Timeout);
	bool IsPrecachePSOCompleted() const;

	// Returns true when all components in the current create pass have been attempted.
	// Components whose proxy was deferred (e.g., DelayUntilPSOPrecached) are counted as attempted.
	// Use IsCreateRenderStateFullyCompleted() to check if all proxies actually exist.
	bool IsInitialCreateRenderStatePassCompleted() const;

	// Returns true when all render state creation is truly done, including PSO-deferred proxies.
	bool IsCreateRenderStateFullyCompleted() const;

	bool IsDestroyRenderStateCompleted() const;

	void CollectAssetReferences();
	void SerializeComponentClusters(FArchive& Ar);
#if WITH_EDITOR
	void SerializeComponentClustersWithFilter(FArchive& Ar, TFunctionRef<bool(const FFastGeoComponent&)> ShouldSerializeComponent);
#endif
	void RegisterToNavigationSystem();
	void UnregisterFromNavigationSystem();
	void OnNavigationInitDone(const UNavigationSystemBase& NavSys);
	void RegisterBodyInstances();
	void UnregisterBodyInstances();
	void OnRegisterCompleted();
	void OnUnregisterCompleted();
	void TryCompleteRegistration();
	void TryCompleteUnregistration();
	void ApplyRegistrationTargetState();
	void StartRegisterTransition();
	void StartUnregisterTransition();

	template <typename ThisType, typename TComponentCluster, typename TFunc>
	static void ForEachComponentCluster(ThisType* Self, TFunc&& InFunc)
	{
		auto ForEachArray = [&InFunc](auto& InArray)
		{
			typedef typename TDecay<decltype(InArray)>::Type ArrayType;
			using TArrayComponentCluster = typename ArrayType::ElementType;

			if constexpr (std::is_same<TComponentCluster, FFastGeoComponentCluster>::value)
			{
				for (auto& ComponentCluster : InArray)
				{
					InFunc(ComponentCluster);
				}
			}
			else if (TArrayComponentCluster::Type.IsA(TComponentCluster::Type))
			{
				for (auto& ComponentCluster : InArray)
				{
					checkSlow(ComponentCluster.template IsA<TComponentCluster>());
					Forward<TFunc>(InFunc)(*StaticCast<TComponentCluster*>(&ComponentCluster));
				}
			}
		};

		ForEachArray(Self->ComponentClusters);
		ForEachArray(Self->HLODs);
	}

	template <typename ThisType, typename TComponentCluster, typename TFunc>
	static bool ForEachComponentClusterBreakable(ThisType* Self, TFunc&& InFunc)
	{
		auto ForEachArray = [&InFunc](auto& InArray) -> bool
		{
			typedef typename TDecay<decltype(InArray)>::Type ArrayType;
			using TArrayComponentCluster = typename ArrayType::ElementType;

			if constexpr (std::is_same<TComponentCluster, FFastGeoComponentCluster>::value)
			{
				for (auto& ComponentCluster : InArray)
				{
					if (!InFunc(ComponentCluster))
					{
						return false;
					}
				}
			}
			else if (TArrayComponentCluster::Type.IsA(TComponentCluster::Type))
			{
				for (auto& ComponentCluster : InArray)
				{
					checkSlow(ComponentCluster.template IsA<TComponentCluster>());
					if (!Forward<TFunc>(InFunc)(*StaticCast<TComponentCluster*>(&ComponentCluster)))
					{
						return false;
					}
				}
			}

			return true;
		};

		return ForEachArray(Self->ComponentClusters) &&
			ForEachArray(Self->HLODs);
	}

	// UFastGeoContainer's physics state
	enum class EPhysicsState : uint8
	{
		None,
		Creating,
		Created,
		Destroying
	};
	EPhysicsState PhysicsState = EPhysicsState::None;

	// UFastGeoContainer's registration state
	enum class ERegistrationState : uint8
	{
		Unregistered,
		Registering,
		Registered,
		Unregistering
	};
	ERegistrationState RegistrationState = ERegistrationState::Unregistered;

	// UFastGeoContainer's registration target state
	enum class ERegistrationTargetState : uint8
	{
		Registered,
		Unregistered
	};
	ERegistrationTargetState RegistrationTargetState = ERegistrationTargetState::Unregistered;

	// Persistent data
	TArray<FFastGeoComponentCluster> ComponentClusters;
	TArray<FFastGeoHLOD> HLODs;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> Assets;

	// Pre-computed unique material list, built at cook time in PreSave.
	// Base materials (OverrideMaterials, mesh slots, decal, light function) are hard refs.
	// Intentionally NOT a UPROPERTY: every material in this array is already GC-anchored
	// transitively via Assets (component overrides serialized into Assets directly, mesh
	// slot defaults reachable through UStaticMesh/USkinnedAsset entries in Assets). Skipping
	// the UPROPERTY shaves a duplicate UObject* traversal pass per container during GC mark.
	// Serialized manually in UFastGeoContainer::Serialize.
	TArray<TObjectPtr<UMaterialInterface>> MaterialAssetHardRefs;

	// Nanite override materials are soft refs - may be stripped on non-Nanite platforms
	// (FMaterialOverrideNanite::Serialize can null them during cook).
	// Resolved in CacheMaterialAssets when loading (Serialize) cooked container.
	UPROPERTY()
	TArray<TSoftObjectPtr<UMaterialInterface>> MaterialAssetNaniteOverrideSoftRefs;

	// Bitmask over Assets: set for entries referenced by non-render-only components.
	// Computed at cook time. On non-rendering processes, unset entries are nulled at load time.
	TBitArray<> NonRenderOnlyAssetMask;

	UPROPERTY()
	TObjectPtr<AFastGeoSurrogateActor> SurrogateActor;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	bool bPSOPrecachingInitiated = false;
#endif

	// Resolved material cache (transient).
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInterface>> CachedMaterialAssets;
	bool bMaterialAssetsCached = false;
	bool bHasStrippedRenderOnlyComponents = false;

	// Transient data
	TArray<IPrimitiveComponent*> PrimitiveComponents;
	TArray<IStaticMeshComponent*> StaticMeshComponents;
	TArray<FFastGeoComponent*> CollisionComponents;
	TArray<FNavigationElementHandle> NavigationElementHandles;

	bool bIsBeingAsyncDestroyed = false;
	int32 RegistrationEpoch = 0;

	// Prerequisites: must be initialized on GT before starting render/physics work.
	// Workers read from AsyncContext instead of performing lookups not safe from generic workers.
	FFastGeoAsyncContext AsyncContext;
	FFastGeoMaterialPostLoadInit MaterialPostLoadInit;
	FFastGeoMaterialRelevanceInit MaterialRelevanceInit;

	FRenderCommandFence DestroyFence;

	FOnRegistered OnRegistered;
	FOnUnregistered OnUnregistered;

	friend class UFastGeoWorldSubsystem;
	friend class UFastGeoWorldPartitionRuntimeCellTransformer;
	friend class FFastGeoGatherFastGeoContainerAssetRefsArchive;
	friend class FFastGeoComponent;
	friend struct FFastGeoRegisteredComponent;
	friend struct FFastGeoAsyncContext;
	friend struct FFastGeoMaterialPostLoadInit;
	friend struct FFastGeoMaterialRelevanceInit;
};