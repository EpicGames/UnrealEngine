// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoComponent.h"
#include "DeferredDecalSceneProxyDesc.h"
#include "Engine/TimerHandle.h"

class FDeferredDecalProxy;

class FASTGEOSTREAMING_API FFastGeoDecalComponent : public FFastGeoComponent
{
public:
	typedef FFastGeoComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoDecalComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	FFastGeoDecalComponent(const FFastGeoDecalComponent& Other);
	virtual ~FFastGeoDecalComponent() = default;

	//~ Begin FFastGeoComponent interface
	virtual void ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	virtual bool ShouldComponentAddToRenderScene() const override;
	virtual void CreateRenderState(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState(FFastGeoDestroyRenderStateContext* Context) override;
	virtual void UpdateVisibility() override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual bool ShouldPrecachePSOs() const override;
	virtual void PrecachePSOs_Concurrent() override;
	virtual FPSOPrecacheComponentData& GetPSOPrecacheComponentData() override { return PSOPrecacheComponentData; }
	virtual const FPSOPrecacheComponentData& GetPSOPrecacheComponentData() const override { return PSOPrecacheComponentData; }
#endif
	//~ End FFastGeoComponent interface

protected:
	FBoxSphereBounds CalcBounds() const;
	FTransform GetTransformIncludingDecalSize() const;
	FDeferredDecalProxy* CreateSceneProxy(ESceneProxyCreationError& OutError);
	void InitializeSceneProxyDescDynamicProperties();
	void UpdateVisibilityInternal();
	void SetLifeSpan(const float InLifeSpan);

private:
	// Persistent data
	FVector DecalSize;
	FDeferredDecalSceneProxyDesc SceneProxyDesc{};
	uint8 bIsVisible : 1 = true;
	uint8 bDestroyOwnerAfterFade : 1 = false;

	// Transient runtime state
	uint8 bFadeCompleted : 1 = false;
	FDeferredDecalProxy* SceneProxy = nullptr;
	FTimerHandle TimerHandle_DestroyDecalComponent;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;
#endif
};