// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class FSceneInterface;
class FDeferredDecalProxy;
class FPrimitiveSceneProxy;

class FFastGeoDestroyRenderStateContext
{
public:
	FFastGeoDestroyRenderStateContext(FSceneInterface* InScene);
	~FFastGeoDestroyRenderStateContext();

	static void DestroyProxy(FFastGeoDestroyRenderStateContext* InContext, FPrimitiveSceneProxy* InPrimitiveSceneProxy, FSceneInterface* InScene);
	static void DestroyProxy(FFastGeoDestroyRenderStateContext* InContext, FDeferredDecalProxy* InDeferredDecalProxy, FSceneInterface* InScene);

private:
	FSceneInterface* Scene;
	TArray<FPrimitiveSceneProxy*> PrimitiveSceneProxies;
	TArray<FDeferredDecalProxy*> DeferredDecalProxies;
};