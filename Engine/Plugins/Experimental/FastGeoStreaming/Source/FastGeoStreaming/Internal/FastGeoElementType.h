// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FFastGeoElementType
{
public:
	static const FFastGeoElementType Invalid;

	explicit FFastGeoElementType(const FFastGeoElementType* Parent = nullptr) : ID(++NextUniqueID), ParentType(Parent) {}

	bool operator==(const FFastGeoElementType& Other) const
	{
		return IsSameTypeID(Other.ID);
	}

	bool IsA(const FFastGeoElementType& Other) const
	{
		return IsSameTypeID(Other.ID) || (ParentType && ParentType->IsA(Other));
	}

	/** Returns true if element has the same type ID than the specified type */
	bool IsSameTypeID(uint32 InID) const
	{
		return ID == InID;
	}

	bool IsValid() const
	{
		return !IsSameTypeID(Invalid.ID);
	}

private:
	enum class EInitMode { Invalid };
	explicit FFastGeoElementType(EInitMode) : ID(0xFFFFFFFF), ParentType(nullptr) {}

	static uint32 NextUniqueID;
	uint32 ID;
	const FFastGeoElementType* ParentType;

	friend class IFastGeoElement;
};