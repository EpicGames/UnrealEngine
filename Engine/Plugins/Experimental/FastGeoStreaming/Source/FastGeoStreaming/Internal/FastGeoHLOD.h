// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoComponentCluster.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODObject.h"

class FFastGeoHLOD : public FFastGeoComponentCluster, public IWorldPartitionHLODObject
{
public:
	typedef FFastGeoComponentCluster Super;

	/** Static type identifier for this element class */
	FASTGEOSTREAMING_API static const FFastGeoElementType Type;

	FFastGeoHLOD(UFastGeoContainer* InOwner = nullptr, FName InName = NAME_None, FFastGeoElementType InType = Type);
	~FFastGeoHLOD() = default;

	virtual bool IsVisible() const override { return bIsVisible; }

	// ~Begin FFastGeoComponentCluster interface
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	virtual void Serialize(FArchive& Ar) override;
	// ~End FFastGeoComponentCluster interface

	// ~Begin IWorldPartitionHLODObject interface
	virtual UObject* GetUObject() const override { return nullptr; }
	virtual ULevel* GetHLODLevel() const override;
	virtual FString GetHLODNameOrLabel() const override;
	virtual bool DoesRequireWarmup() const override;
	virtual TSet<UObject*> GetAssetsToWarmup() const override;
	virtual void SetVisibility(bool bIsVisible) override;
	virtual const FGuid& GetSourceCellGuid() const override;
	virtual bool IsStandalone() const override;
	virtual const FGuid& GetStandaloneHLODGuid() const override;
	virtual bool IsCustomHLOD() const override;
	virtual const FGuid& GetCustomHLODGuid() const override;
	// ~End IWorldPartitionHLODObject interface

#if WITH_EDITOR
	// ~Begin FFastGeoComponentCluster interface
	virtual FFastGeoComponent& AddComponent(FFastGeoElementType InComponentType) override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	// ~End FFastGeoComponentCluster interface

	void SetSourceCellGuid(const FGuid& InSourceCellGuid);
	void SetRequireWarmup(bool bInRequireWarmup);
	void SetStandaloneHLODGuid(const FGuid& InStandaloneHLODGuid);
	void SetCustomHLODGuid(const FGuid& InCustomHLODGuid);
#endif

private:

	// Transient Data
	bool bIsVisible;

	// Persistent Data
	bool bRequireWarmup;
	FGuid SourceCellGuid;

	enum class EHLODGuidType : uint8
	{
		None,
		Standalone,
		Custom
	};
	FGuid HLODGuid;
	EHLODGuidType HLODGuidType = EHLODGuidType::None;

	const FGuid& GetHLODGuid(EHLODGuidType InGuidType) const;
	void SetHLODGuid(const FGuid& InGuid, EHLODGuidType InGuidType);
};
