// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoSkinnedMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "InstanceData/InstanceDataManager.h"
#include "InstancedSkinnedMeshSceneProxyDesc.h"

class FASTGEOSTREAMING_API FFastGeoInstancedSkinnedMeshComponent : public FFastGeoSkinnedMeshComponentBase
{
public:
	typedef FFastGeoSkinnedMeshComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoInstancedSkinnedMeshComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoInstancedSkinnedMeshComponent() = default;

protected:
	//~ Being FFastGeoComponent interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

	//~ Begin FFastGeoPrimitiveComponent interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override { return SceneProxyDesc; }
#if WITH_EDITOR
	virtual void InitializeSceneProxyDescFromComponent(UActorComponent* Component) override;
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
#endif
	virtual void InitializeSceneProxyDescDynamicProperties() override;
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoSkinnedMeshComponentBase interface
	virtual FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual FSkeletalMeshObject* CreateMeshObject() override;
	virtual FPrimitiveSceneProxy* AllocateSceneProxy() override;
	// FInstancedSkeletalMeshObject::Update consumes only LODIndex; skinning runs GPU-side through
	// the AnimBank scene-extension proxy (or static-pose path), so we skip the ref-pose fill.
	virtual void UpdateSkinning() override;

	int32 GetInstanceCount() const
	{
		return InstanceData.Num();
	}

	UTransformProviderData* GetTransformProvider() const
	{
		return SceneProxyDesc.TransformProvider;
	}
	//~ End FFastGeoSkinnedMeshComponentBase interface

	TConstArrayView<FTransform3f> GetInstancePrevTransforms() const { return {}; }

private:
	TArray<FSkinnedMeshInstanceData> InstanceData;
	int32 NumCustomDataFloats = 0;
	TArray<float> InstanceCustomData;

	FInstancedSkinnedMeshSceneProxyDesc SceneProxyDesc;

	TSharedPtr<FInstanceDataManager> InstanceDataManager;

	friend class FInstancedSkinnedMeshComponentHelper;
};
