// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoStaticMeshComponent.h"
#include "InstancedStaticMeshSceneProxyDesc.h"

class FASTGEOSTREAMING_API FFastGeoInstancedStaticMeshComponent : public FFastGeoStaticMeshComponentBase
{
public:
	typedef FFastGeoStaticMeshComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoInstancedStaticMeshComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoInstancedStaticMeshComponent() = default;

	//~ Begin FFastGeoComponent Interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread() override;
#endif
	virtual void OnAsyncCreatePhysicsState() override;
	virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	virtual void OnAsyncDestroyPhysicsState() override;
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread() override;
	virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent Interface

	//~ Begin FFastGeoPrimitiveComponent interface
	virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	virtual FBox GetNavigationBounds() const override;
	virtual bool IsNavigationRelevant() const override;
	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	virtual bool ShouldSkipNavigationDirtyAreaOnAddOrRemove() override { return true; }
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
#if WITH_EDITOR
	virtual void ReserveSurrogateInstanceBodyIndices(FFastGeoSurrogateBodyInstanceIndex& InOutNextAvailableInstanceBodyIndex) override;
#endif
	//~ End FFastGeoPrimitiveComponent interface

	void ForEachInstanceMatrix(TFunctionRef<void(const FMatrix&)> Func) const;
	int32 GetInstanceCount() const;

protected:
	//~ Begin FFastGeoPrimitiveComponent interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override { return SceneProxyDesc; }
#if WITH_EDITOR
	virtual void InitializeSceneProxyDescFromComponent(UActorComponent* Component) override;
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
#endif
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	virtual void GetBodyInstances(TArray<FBodyInstance*>& OutBodyInstances) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoStaticMeshComponentBase interface
	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
#endif
	//~ End FFastGeoStaticMeshComponentBase interface

private:
	enum class EBoundsType
	{
		LocalBounds,
		WorldBounds,
		NavigationBounds
	};
	FBoxSphereBounds CalculateBounds(EBoundsType BoundsType);
	void BuildInstanceData();
	void CreateAllInstanceBodies();
	static TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects(const TArray<FBodyInstance*>& InInstanceBodies);
#if WITH_EDITOR
	void InitializePerInstanceSMData(const TArray<FInstancedStaticMeshInstanceData>& InPerInstanceSMData);
#endif

private:
	// Persistent Data
	// ------------------------------
	TArray<FInstancedStaticMeshInstanceData> HighPrecisionPerInstanceSMData;
	TArray<FTransform3f> LowPrecisionPerInstanceSMData;
	int32 LastInstanceBodyIndex = INDEX_NONE;
	int32 InstancingRandomSeed = 0;
	TArray<float> PerInstanceSMCustomData;
	TArray<FInstancedStaticMeshRandomSeed> AdditionalRandomSeeds;
	FBox NavigationBounds;
	TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> SpatialHashes;
	TArray<float> PerInstanceRandomIDs;
	FInstancedStaticMeshSceneProxyDesc SceneProxyDesc{};
	bool bUseHighPrecisionPerInstanceSMData : 1 = false;

	// Transient data
	// ------------------------------
	bool bAppliedWorldTransform : 1 = false;
	/** Physics representation of the instance bodies. */
	TArray<FBodyInstance*> InstanceBodies;
	/** Payload used by asynchronous destruction of physics state (see OnAsyncDestroyPhysicsState). */
	TArray<FBodyInstance*> AsyncDestroyPhysicsStatePayload;

	friend class FInstancedStaticMeshComponentHelper;
};
