// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoComponent.h"
#include "LocalLightSceneProxyDesc.h"

class FLightSceneProxy;

class FASTGEOSTREAMING_API FFastGeoLocalLightComponent : public FFastGeoComponent
{
public:
	typedef FFastGeoComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoLocalLightComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	FFastGeoLocalLightComponent(const FFastGeoLocalLightComponent& Other);
	virtual ~FFastGeoLocalLightComponent() = default;

	//~ Begin FFastGeoComponent interface
	virtual void ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const override;
	virtual void Serialize(FArchive& Ar) override;
	virtual bool ShouldComponentAddToRenderScene() const override;
	virtual void CreateRenderState(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState(FFastGeoDestroyRenderStateContext* Context) override;
	virtual void UpdateVisibility() override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual bool ShouldPrecachePSOs() const override;
	virtual void PrecachePSOs_Concurrent() override;
	virtual FPSOPrecacheComponentData& GetPSOPrecacheComponentData() override { return PSOPrecacheComponentData; }
	virtual const FPSOPrecacheComponentData& GetPSOPrecacheComponentData() const override { return PSOPrecacheComponentData; }
#endif

protected:
	virtual FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() = 0;
	virtual const FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() const = 0;
	virtual FLightSceneProxy* CreateTypedSceneProxy() = 0;

	void InitializeSceneProxyDescDynamicProperties();
	bool UpdateVisibilityInternal();

private:
	FLightSceneProxy* CreateSceneProxy(ESceneProxyCreationError& OutError);

	// Persistent data
	bool bAffectsWorld = true;
	bool bSourceComponentIsVisible = true;

	// Transient data
	bool bIsVisible = true;
	FLightSceneProxy* SceneProxy = nullptr;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;
#endif
};