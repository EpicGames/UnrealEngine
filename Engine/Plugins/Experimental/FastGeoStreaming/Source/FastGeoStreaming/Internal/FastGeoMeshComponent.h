// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoPrimitiveComponent.h"
#include "Components/MeshComponent.h"
#include "MeshComponentHelper.h"

class FASTGEOSTREAMING_API FFastGeoMeshComponent : public FFastGeoPrimitiveComponent
{
public:
	typedef FFastGeoPrimitiveComponent Super;
	
	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoMeshComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoMeshComponent() = default;

	//~ Begin FFastGeoComponent interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* InComponent) override;
#endif
	//~ End FFastGeoComponent interface

	FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const;

	void GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const;

	//Get the asset material slot overlay material
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& AssetMaterialSlotOverlayMaterials) const = 0;

	virtual const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const = 0;
	virtual UMaterialInterface* GetOverlayMaterial() const = 0;

	TConstArrayView<TObjectPtr<UMaterialInterface>> GetOverrideMaterials() const
	{
		return OverrideMaterials;
	}
protected:
	// Persistent Data
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

private:
	friend class FMeshComponentHelper;
};