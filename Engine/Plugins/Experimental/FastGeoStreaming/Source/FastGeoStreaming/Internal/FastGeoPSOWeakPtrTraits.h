// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSOPrecacheComponent.h"
#include "FastGeoWeakElement.h"

#if UE_WITH_PSO_PRECACHING

// Specialization: if T : FFastGeoComponent -> use FWeakFastGeoComponent
template<typename T>
struct TPSOWeakPtrTraits<T, typename TEnableIf<TIsDerivedFrom<T, FFastGeoComponent>::Value>::Type>
{
	using WeakType = FWeakFastGeoComponent;

	static WeakType Make(T* Ptr) { return WeakType(Ptr); }
	static T* Get(const WeakType& W) { return W.Get<T>(); }
};

#endif // UE_WITH_PSO_PRECACHING
