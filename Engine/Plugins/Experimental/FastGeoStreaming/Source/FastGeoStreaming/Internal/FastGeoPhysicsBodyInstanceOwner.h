// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosUserEntity.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"

class FFastGeoPrimitiveComponent;
class UFastGeoContainer;
struct FHitResult;
struct FOverlapResult;

class FFastGeoPhysicsBodyInstanceOwner : public FChaosUserDefinedEntity, public IPhysicsBodyInstanceOwner
{
public:
	FFastGeoPhysicsBodyInstanceOwner();
	virtual ~FFastGeoPhysicsBodyInstanceOwner() = default;

	/** Returns the IPhysicsBodyInstanceOwner based on the provided FChaosUserDefinedEntity */
	static IPhysicsBodyInstanceOwner* GetPhysicsBodyInstanceOwner(FChaosUserDefinedEntity* InUserDefinedEntity);

	/** Returns the typed FFastGeoPhysicsBodyInstanceOwner from a hit result, or nullptr if the hit did not involve FastGeo geometry or the physics object/owner references are no longer valid. */
	static FASTGEOSTREAMING_API const FFastGeoPhysicsBodyInstanceOwner* FromHitResult(const FHitResult& HitResult);

	/** Returns the typed FFastGeoPhysicsBodyInstanceOwner from an overlap result, or nullptr if the overlap did not involve FastGeo geometry or the physics object/owner references are no longer valid. */
	static FASTGEOSTREAMING_API const FFastGeoPhysicsBodyInstanceOwner* FromOverlapResult(const FOverlapResult& OverlapResult);

	//~ Begin FChaosUserDefinedEntity interface
	TWeakObjectPtr<UObject> GetOwnerObject() override;
	//~ End FChaosUserDefinedEntity interface

	//~ Begin IPhysicsBodyInstanceOwner interface
public:
	virtual bool IsStaticPhysics() const override;
	virtual bool IsMultiBodyOverlap() const override;
	virtual UObject* GetSourceObject() const override;
	virtual FTransform GetPhysicsOwnerTransform() const override;
	virtual ECollisionChannel GetCollisionObjectType() const override;
	virtual ECollisionEnabled::Type GetCollisionEnabled() const override;
	virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const override;
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	virtual UPhysicalMaterial* GetPhysicsMaterialOverride() const override;
	virtual UMaterialInterface* GetPhysicsMaterialBase() const override;
	virtual int32 GetNumMaterials() const override;
	virtual UMaterialInterface* GetMaterial(int32 Index) const override;
	virtual UBodySetup* GetPhysicsBodySetup() const override;
	virtual UObject* GetSourceObjectOwner() const override;
	virtual FTransform GetPhysicsOwnerSocketTransform(FName InSocketName) const override;
	virtual bool IsPhysicsOwnerMovable() const override;
	virtual bool IsPhysicsOwnerSimulatingPhysics() const override;
	virtual FVector GetPhysicsOwnerVelocity() const override;
	virtual UObject* GetPhysicsOwnerAttachmentRoot() const override;
	virtual bool IsPhysicsObjectWorldGeometry() const override;
	virtual bool DoesSocketExistOnPhysicsOwner(FName InSocketName) const override;
	virtual const FWalkableSlopeOverride& GetWalkableSlopeOverride() const override;
	//~ End IPhysicsBodyInstanceOwner interface

	FFastGeoPrimitiveComponent* GetOwnerComponent() const { check(OwnerContainer.IsValid()); return OwnerComponent; }
	UFastGeoContainer* GetOwnerContainer() const { check(OwnerContainer.IsValid()); return OwnerContainer.Get(); }

private:
	static const FFastGeoPhysicsBodyInstanceOwner* FromPhysicsObject(Chaos::FConstPhysicsObjectHandle PhysicsObject);

	void Uninitialize();
	void Initialize(FFastGeoPrimitiveComponent* InOwner);

	FFastGeoPrimitiveComponent* OwnerComponent;
	TWeakObjectPtr<UFastGeoContainer> OwnerContainer;
	static const FName NAME_FastGeoPhysicsBodyInstanceOwner;

	friend class FFastGeoPrimitiveComponent;
	friend class FFastGeoInstancedStaticMeshComponent;
};