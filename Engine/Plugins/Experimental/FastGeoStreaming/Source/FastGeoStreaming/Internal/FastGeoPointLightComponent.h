// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoLocalLightComponent.h"
#include "PointLightSceneProxyDesc.h"

class FLightSceneProxy;

class FASTGEOSTREAMING_API FFastGeoPointLightComponentBase : public FFastGeoLocalLightComponent
{
public:
	typedef FFastGeoLocalLightComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoPointLightComponentBase(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoPointLightComponentBase() = default;

	//~ Begin FFastGeoComponent interface
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual bool ShouldPrecachePSOs() const override;
#endif
	//~ End FFastGeoComponent interface

protected:
	//~ Begin FFastGeoComponent interface
	bool ShouldComponentAddToRenderScene() const;
	//~ End FFastGeoComponent interface

	//~ Begin FFastGeoLocalLightComponent interface
	virtual FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() override { return GetPointLightSceneProxyDesc(); }
	virtual const FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() const override { return GetPointLightSceneProxyDesc(); }
	//~ End FFastGeoLocalLightComponent interface

	virtual FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() = 0;
	virtual const FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() const = 0;

	bool IsPointLightSupported() const;
};

class FASTGEOSTREAMING_API FFastGeoPointLightComponent : public FFastGeoPointLightComponentBase
{
public:
	typedef FFastGeoPointLightComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoPointLightComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoPointLightComponent() = default;

	//~ Begin FFastGeoComponent interface
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

protected:
	//~ Begin FFastGeoLightComponent interface
	virtual FLightSceneProxy* CreateTypedSceneProxy() override;
	//~ End FFastGeoLightComponent interface

	//~ Begin FFastGeoPointLightComponentBase interface
	virtual FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() const override { return SceneProxyDesc; }
	//~ End FFastGeoPointLightComponentBase interface

private:
	FPointLightSceneProxyDesc SceneProxyDesc{};
};