// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoComponent.h"
#include "FastGeoPhysicsBodyInstanceOwner.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/EngineTypes.h"
#include "PrimitiveSceneInfoData.h"
#include "PrimitiveSceneProxy.h"
#include "RenderCommandFence.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Templates/DontCopy.h"

class FRegisterComponentContext;
class FPrimitiveSceneProxy;
class UPrimitiveComponent;
class URuntimeVirtualTexture;
class UFastGeoContainer;
class UFastGeoSurrogateComponent;
class FFastGeoPrimitiveComponent;
class FFastGeoDestroyRenderStateContext;
struct FFastGeoSurrogateBodyInstanceIndex;
struct FPrimitiveSceneDesc;
struct FPrimitiveSceneProxyDesc;
struct FPSOPrecacheParams;
enum class EPSOPrecachePriority : uint8;

class FASTGEOSTREAMING_API FFastGeoPrimitiveComponent : public FFastGeoComponent, public IPrimitiveComponent
{
public:
	typedef FFastGeoComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoPrimitiveComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoPrimitiveComponent() = default;
	FFastGeoPrimitiveComponent(const FFastGeoPrimitiveComponent& Other);

	TArray<URuntimeVirtualTexture*> const& GetRuntimeVirtualTextures() const { return RuntimeVirtualTextures; }
	bool IsFirstPersonRelevant() const;

	FPrimitiveComponentId GetPrimitiveSceneId() const { return PrimitiveSceneData.PrimitiveSceneId; }
	virtual UObject const* AdditionalStatObject() const { return nullptr; }
		
	FMatrix GetRenderMatrix() const;

	// Used by FStaticMeshComponentHelper/FInstancedStaticMeshComponentHelper
	FTransform GetComponentTransform() const { return GetTransform(); }

#if WITH_EDITOR
	virtual void NotifyRenderStateChanged();
#endif
	bool IsDrawnInGame() const;
	void SetCollisionEnabled(bool bInCollisionEnabled);
	void SetCustomPrimitiveData(TConstArrayView<float> InCustomPrimitiveData);
	void UpdateVisibility();
	EComponentMobility::Type GetMobility() const;
	UFastGeoSurrogateComponent* GetSurrogateComponent() const;

	//~ Begin IPrimitiveComponent interface
	virtual bool IsRenderStateDirty() const override;
	virtual bool IsRenderStateCreated() const override;
	virtual bool ShouldCreateRenderState() const override;
	virtual bool IsRegistered() const override;
	virtual bool IsUnreachable() const override;
	virtual bool IsStaticMobility() const override;
	virtual bool IsMipStreamingForced() const override;
	virtual UWorld* GetWorld() const override;
	virtual FSceneInterface* GetScene() const override;
	virtual FPrimitiveSceneProxy* GetSceneProxy() const override;
	virtual void MarkRenderStateDirty() override;
	virtual void DestroyRenderState() override;
	virtual void CreateRenderState(FRegisterComponentContext* Context) override;
	virtual FString GetName() const override;
	virtual FString GetFullName() const override;
	virtual FTransform GetTransform() const override;
	virtual const FBoxSphereBounds& GetBounds() const override;
	float GetLastRenderTimeOnScreen() const override;
	virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const override;
	virtual UObject* GetUObject() override;
	virtual const UObject* GetUObject() const override;
	virtual void PrecachePSOs() override;
	virtual UObject* GetOwner() const override;
	virtual FString GetOwnerName() const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FRenderAssetOwnerStreamingState& GetStreamingState() const override;
	virtual ULevel* GetComponentLevel() const override;
	virtual IPrimitiveComponent* GetLODParentPrimitive() const override;
	virtual float GetMinDrawDistance() const override;
	virtual float GetStreamingScale() const override;
	virtual void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex) override;
	virtual UStreamableRenderAsset* GetStreamableNaniteAsset() const override;
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
#if WITH_EDITOR
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) override;
#endif // WITH_EDITOR
	virtual HHitProxy* CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
	//~ End IPrimitiveComponent interface

	//~ Begin FFastGeoComponent interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void InitializeDynamicProperties() override;
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread() override;
	virtual void OnAsyncCreatePhysicsState() override;
	virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread() override;
	virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
	virtual void OnAsyncDestroyPhysicsState() override;
	virtual bool IsCollisionEnabled() const override;
	virtual bool ShouldComponentAddToRenderScene() const override;
	virtual void DestroyRenderState(FFastGeoDestroyRenderStateContext* Context) override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void PrecachePSOs_Concurrent() override;
	virtual FPSOPrecacheComponentData& GetPSOPrecacheComponentData() override { return PSOPrecacheComponentData; }
	virtual const FPSOPrecacheComponentData& GetPSOPrecacheComponentData() const override { return PSOPrecacheComponentData; }
	virtual bool NeedsPSORecreate() const override { return bCreatedWithPSOFallbackMaterial; }
#endif
	bool CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority = EPSOPrecachePriority::High);
	//~End FFastGeoComponent interface

	//~ Begin Materials
	virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const = 0;
	virtual int32 GetNumMaterials() const = 0;
	UE_DEPRECATED(5.7, "Please use GetUsedMaterialPropertyDesc with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(ERHIFeatureLevel::Type FeatureLevel) const;
	FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(EShaderPlatform InShaderPlatform) const;
	//~ End Materials

	//~ Begin Navigation
	virtual bool IsNavigationRelevant() const;
	virtual bool ShouldSkipNavigationDirtyAreaOnAddOrRemove() { return false; }
	virtual FBox GetNavigationBounds() const;
	virtual void GetNavigationData(FNavigationRelevantData& OutData) const;
	virtual EHasCustomNavigableGeometry::Type HasCustomNavigableGeometry() const;
	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const { return true; }
	//~ End Navigation

#if WITH_EDITOR
	virtual void ReserveSurrogateInstanceBodyIndices(FFastGeoSurrogateBodyInstanceIndex& InOutNextAvailableInstanceBodyIndex);
#endif

protected:
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() = 0;
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const = 0;
#if WITH_EDITOR
	virtual void InitializeSceneProxyDescFromComponent(UActorComponent* Component) = 0;
	virtual void ResetSceneProxyDescUnsupportedProperties();
#endif
	virtual void InitializeSceneProxyDescDynamicProperties();
	virtual FPrimitiveSceneProxy* CreateSceneProxy(ESceneProxyCreationError* OutError) = 0;
	virtual FPrimitiveSceneDesc BuildSceneDesc();

	//~ Begin Physics
	virtual bool IsStaticPhysics() const;
	virtual bool IsMultiBodyOverlap() const;
	virtual UObject* GetSourceObject() const;
	virtual ECollisionChannel GetCollisionObjectType() const;
	virtual ECollisionEnabled::Type GetCollisionEnabled() const;
	virtual UPhysicalMaterial* GetPhysicsMaterialOverride() const;
	virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const;
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const;
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const;
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const;

	UE_DEPRECATED(5.8, "GetPhysicalMaterial was removed.")
	virtual UPhysicalMaterial* GetPhysicalMaterial() const { return nullptr; }
	//~ End Physics

protected:
	// Persistent data
	FBoxSphereBounds LocalBounds;
	FBoxSphereBounds WorldBounds;
	uint8 bIsVisible : 1 = true;
	uint8 bStaticWhenNotMoveable : 1 = true;
	uint8 bFillCollisionUnderneathForNavmesh : 1 = false;
	uint8 bRasterizeAsFilledConvexVolume : 1 = false;
	uint8 bCanEverAffectNavigation : 1 = false;
	uint8 bMultiBodyOverlap : 1 = false;
	int32 SurrogateComponentDescriptorIndex = INDEX_NONE;
	FCustomPrimitiveData CustomPrimitiveData;
	TEnumAsByte<EHasCustomNavigableGeometry::Type> bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::No;
	FBodyInstance BodyInstance;
	TArray<URuntimeVirtualTexture*> RuntimeVirtualTextures;

	// Runtime Data (transient)
	FFastGeoPhysicsBodyInstanceOwner BodyInstanceOwner;
	FPrimitiveSceneInfoData PrimitiveSceneData{};
	/** Payload used to release BodyInstance resources in asynchronous mode (see OnAsyncDestroyPhysicsState). */
	TOptional<FBodyInstance::FAsyncTermBodyPayload> AsyncTermBodyPayload;

protected:
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) {}
	void SetupPrecachePSOParams(FPSOPrecacheParams& Params);

	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;

	// True when the proxy was created with a fallback material (UseFallbackMaterialUntilPSOPrecached).
	// Used by OnComponentPSOPrecacheCompleted to know this component needs a recreate when PSO completes.
	bool bCreatedWithPSOFallbackMaterial = false;
#endif

private:
	bool UsePSOPrecacheFallbackMaterial() const;

	friend class UFastGeoWorldPartitionRuntimeCellTransformer;
	friend class FFastGeoComponentCluster;
	friend class FPrimitiveComponentHelper;
	friend class FFastGeoPhysicsBodyInstanceOwner;
	friend class UFastGeoContainer;
};
