// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoStaticMeshComponent.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;

struct FProceduralISMComponentDescriptor;

class FFastGeoProceduralISMComponent : public FFastGeoStaticMeshComponentBase
{
public:
	typedef FFastGeoStaticMeshComponentBase Super;

	/** Static type identifier for this element class */
	FASTGEOSTREAMING_API static const FFastGeoElementType Type;

	FFastGeoProceduralISMComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoProceduralISMComponent() = default;

	FASTGEOSTREAMING_API void InitializeFromComponentDescriptor(const FProceduralISMComponentDescriptor& InDescriptor);

	/** Set spatial hashes which provide bounds for sub-ranges of instances within this primitive, enabling early CPU culling. */
	FASTGEOSTREAMING_API void SetSpatialHashes(TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>&& InSpatialHashes);

#if WITH_EDITOR
	/** Sets an actor to use as the hit proxy target when this component is clicked in the editor.
	 *  If set, clicking this primitive in the editor will select InActor instead of the container.
	 *  Must be called before Register().
	 */
	FASTGEOSTREAMING_API void SetHitProxyTargetActor(AActor* InActor);

	FASTGEOSTREAMING_API void UpdateEditorSelectionState();
#endif

	//~ Begin FFastGeoComponent Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End FFastGeoComponent Interface

protected:
	//~ Begin FFastGeoPrimitiveComponent interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual FPrimitiveSceneDesc BuildSceneDesc() override;
	virtual bool IsNavigationRelevant() const override { return false; }
#if WITH_EDITOR
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) override;
#endif
	virtual void InitializeSceneProxyDescDynamicProperties() override;
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	virtual HHitProxy* CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoStaticMeshComponentBase interface
	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
#endif
	//~ End FFastGeoStaticMeshComponentBase interface

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe>& BuildInstanceData();

private:
	// Persistent Data
	int32 NumInstances = 0;
	int32 NumCustomDataFloats = 0;
	FBox PrimitiveBoundsOverride = FBox(ForceInit);
	FInstancedStaticMeshSceneProxyDesc SceneProxyDesc{};

	// Transient data
	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> DataProxy{};
	TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> SpatialHashes;
#if WITH_EDITOR
	TWeakObjectPtr<AActor> HitProxyTargetActor;
#endif

	friend class FInstancedStaticMeshComponentHelper;
};
