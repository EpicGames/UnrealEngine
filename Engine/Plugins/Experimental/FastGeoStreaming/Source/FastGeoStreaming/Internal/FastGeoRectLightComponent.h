// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoLocalLightComponent.h"
#include "RectLightSceneProxyDesc.h"

class FASTGEOSTREAMING_API FFastGeoRectLightComponent : public FFastGeoLocalLightComponent
{
public:
	typedef FFastGeoLocalLightComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoRectLightComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoRectLightComponent() = default;

	//~ Begin FFastGeoComponent interface
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

protected:
	//~ Begin FFastGeoLightComponent interface
	virtual FLightSceneProxy* CreateTypedSceneProxy() override;
	//~ End FFastGeoLightComponent interface

	//~ Begin FFastGeoLocalLightComponent interface
	virtual FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FLocalLightSceneProxyDesc& GetLocalLightSceneProxyDesc() const override { return SceneProxyDesc; }
	//~ End FFastGeoLocalLightComponent interface

private:
	FRectLightSceneProxyDesc SceneProxyDesc{};
};
