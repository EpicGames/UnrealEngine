// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoComponent.h"
#include "FastGeoContainer.h"
#include "FastGeoWeakElement.h"

struct FFastGeoRegisteredComponent
{
	FFastGeoRegisteredComponent(FFastGeoComponent* InComponent)
		: Component(InComponent)
	{
		check(InComponent);
		check(InComponent->GetOwnerContainer());
		RegistrationEpoch = InComponent->GetOwnerContainer()->RegistrationEpoch;
	}

	// Game-thread only -- IsRegistered / IsRegistering below assert IsInGameThread.
	template <class T>
	FORCEINLINE T* TryGetRegistered() const
	{
		if (T* ComponentPtr = Component.Get<T>())
		{
			UFastGeoContainer* Container = ComponentPtr->GetOwnerContainer();
			if (Container &&
				Container->GetWorld() &&
				(Container->IsRegistered() || Container->IsRegistering()) &&
				(Container->RegistrationEpoch == RegistrationEpoch))
			{
				return ComponentPtr;
			}
		}
		return nullptr;
	}

	FORCEINLINE bool operator==(const FFastGeoRegisteredComponent& Other) const
	{
		return Component == Other.Component && RegistrationEpoch == Other.RegistrationEpoch;
	}

private:
	FWeakFastGeoComponent Component;
	int32 RegistrationEpoch = -1;
};