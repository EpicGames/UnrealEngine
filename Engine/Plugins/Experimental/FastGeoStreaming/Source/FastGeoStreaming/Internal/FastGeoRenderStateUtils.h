// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/Timeout.h"

class FFastGeoComponent;

// General-purpose progress tracker for budgeted render state work.
// Used by create, deferred create, recreate, and ProcessPendingRecreate paths.
// Default element type is FFastGeoComponent* (raw pointer). ProcessPendingRecreate uses
// TFastGeoRenderStateBatch<FFastGeoRegisteredComponent> to avoid persisting raw pointers
// across frames -- stale components are validated inline via TryGetRegistered.
template <typename T = FFastGeoComponent*>
struct TFastGeoRenderStateBatch
{
	TFastGeoRenderStateBatch()
	{
		Reset();
	}

	void Reset()
	{
		ComponentsToProcess.Reset();
		NumToProcess = 0;
		NumProcessed = 0;
		TotalNumProcessed = 0;
		bIsInBlockingWait = false;
	}

	bool IsCompleted() const
	{
		return TotalNumProcessed >= ComponentsToProcess.Num();
	}

	TArray<T> ComponentsToProcess;

	int32 NumToProcess;
	int32 NumProcessed;
	int32 TotalNumProcessed;
	bool bIsInBlockingWait = false;
};

using FFastGeoRenderStateBatch = TFastGeoRenderStateBatch<FFastGeoComponent*>;

namespace FastGeo
{
	// Processes components in a batch using ParallelFor with FTaskTagScope (ParallelGT pattern).
	// Used by sync-mode create, destroy, deferred create, recreate, and ProcessPendingRecreate.
	// Returns true if all components were processed, false if timeout expired.
	// WorkFn receives T& (e.g., FFastGeoComponent*& for raw batches, FFastGeoRegisteredComponent& for safe batches).
	template <typename T, typename FWorkFn>
	FORCEINLINE bool AdvanceRenderStateBudgeted(TFastGeoRenderStateBatch<T>& State, const UE::FTimeout& Timeout, FWorkFn&& WorkFn)
	{
		check(IsInGameThread());

		if (State.IsCompleted())
		{
			return true;
		}

		if (Timeout.IsExpired())
		{
			return false;
		}

		std::atomic<int32> NextIndex{ 0 };
		std::atomic<int32> NumProcessed{ 0 };
		static constexpr int32 MinElementsPerTask = 8;
		const int32 NumRemaining = State.NumToProcess;
		const int32 StartIndex = State.TotalNumProcessed;
		const int32 NumWorkers = FTaskGraphInterface::Get().GetNumWorkerThreads();
		const int32 NumTasks = FMath::Clamp(FMath::DivideAndRoundUp(NumRemaining, MinElementsPerTask), 1, NumWorkers);

		ParallelFor(NumTasks, [&](int32 TaskIndex)
		{
			FTaskTagScope Scope(ETaskTag::EParallelGameThread);
			while (true)
			{
				if (Timeout.IsExpired())
				{
					return;
				}

				const int32 LocalIndex = NextIndex.fetch_add(1, std::memory_order_relaxed);
				if (LocalIndex >= NumRemaining)
				{
					return;
				}

				T& Component = State.ComponentsToProcess[StartIndex + LocalIndex];
				WorkFn(Component);

				NumProcessed.fetch_add(1, std::memory_order_relaxed);
			}
		});

		State.NumProcessed = NumProcessed.load(std::memory_order_relaxed);
		State.TotalNumProcessed += State.NumProcessed;
		State.NumToProcess = State.ComponentsToProcess.Num() - State.TotalNumProcessed;
		return State.IsCompleted();
	}
}
