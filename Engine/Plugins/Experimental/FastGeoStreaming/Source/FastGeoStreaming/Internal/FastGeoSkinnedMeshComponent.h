// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationRuntime.h"
#include "Components/SkinnedMeshComponent.h"
#include "FastGeoMeshComponent.h"
#include "SkinnedMeshSceneProxyDesc.h"

class UPhysicsAsset;
class USkinnedAsset;
class FSkeletalMeshObject;
class FSkeletalMeshRenderData;

class FASTGEOSTREAMING_API FFastGeoSkinnedMeshComponentBase : public FFastGeoMeshComponent
{
public:
	typedef FFastGeoMeshComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoSkinnedMeshComponentBase(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoSkinnedMeshComponentBase() = default;

	//~ Begin FFastGeoComponent interface
	virtual void Serialize(FArchive& Ar) override;
	virtual UBodySetup* GetBodySetup() const override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

	//~ Begin FFastGeoPrimitiveComponent interface
	virtual void ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const override;
	virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	virtual int32 GetNumMaterials() const override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	//~ End FFastGeoPrimitiveComponent interface

protected:
	//~ Begin FFastGeoPrimitiveComponent interface
#if WITH_EDITOR
	virtual void InitializeSceneProxyDescFromComponent(UActorComponent* Component) override;
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
#endif
	virtual void InitializeSceneProxyDescDynamicProperties() override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy(ESceneProxyCreationError* OutError = nullptr) override;
	virtual void DestroyRenderState(FFastGeoDestroyRenderStateContext* Context) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FastGeoMeshComponent interface
	virtual UMaterialInterface* GetOverlayMaterial() const override;
	virtual const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const override;
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& AssetMaterialSlotOverlayMaterials) const override;
	//~ End FastGeoMeshComponent interface

	//~ Begin FastGeoSkinnedMeshComponent interface
	virtual FSkeletalMeshObject* CreateMeshObject();
	virtual FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() = 0;
	virtual const FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() const = 0;
	virtual FPrimitiveSceneProxy* AllocateSceneProxy() = 0;
	/** Default implementation builds ref-pose DynamicData and calls RunMeshObjectUpdate. Paths
	 *  whose FSkeletalMeshObject consumes only LODIndex (e.g. the instanced path) override this
	 *  to skip the ref-pose fill. */
	virtual void UpdateSkinning();
	//~ End FastGeoSkinnedMeshComponent interface

	/** Shared MeshObject->Update plumbing: LOD compute + clamp + asset/render-data validation +
	 *  the Update call itself. Callers provide a pre-built DynamicData; this routine fills in
	 *  the LOD index and does the safety guards. */
	void RunMeshObjectUpdate(FSkinnedMeshSceneProxyDynamicData& InOutDynamicData);

	FBoxSphereBounds CalcMeshBound(const FVector3f& InRootOffset, bool bInUsePhysicsAsset, const FTransform& InLocalToWorld) const;
	USkinnedAsset* GetSkinnedAsset() const { return GetSkinnedMeshSceneProxyDesc().GetSkinnedAsset(); }
	FSkeletalMeshObject* GetMeshObject() { return MeshObject; }
	const FSkeletalMeshObject* GetMeshObject() const { return MeshObject; }
	void DestroyMeshObject();

	bool IsDisallowNanite() const { return GetSkinnedMeshSceneProxyDesc().IsDisallowNanite(); }
	bool IsForceDisableNanite() const { return GetSkinnedMeshSceneProxyDesc().IsForceDisableNanite(); }

	//~ Begin Unsupported
	UPhysicsAsset* GetPhysicsAsset() const { return nullptr; }
	int32 GetPredictedLODLevel() const { return 0; }
	//~ End Unsupported

protected:
	// Persistent data
	TArray<ESkinCacheUsage> SkinCacheUsage;
	bool bOverrideMinLod : 1 = false;
	bool bIncludeComponentLocationIntoBounds = false;
	bool bHideSkin : 1 = false;	
	int32 MinLodModel = 0;

	// Transient
	TArray<FSkelMeshComponentLODInfo> LODInfo;
	FSkeletalMeshObject* MeshObject = nullptr;

	friend class FSkinnedMeshComponentHelper;
};

class FASTGEOSTREAMING_API FFastGeoSkinnedMeshComponent : public FFastGeoSkinnedMeshComponentBase
{
public:
	typedef FFastGeoSkinnedMeshComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoSkinnedMeshComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoSkinnedMeshComponent() = default;
	
protected:
	//~ Begin FFastGeoPrimitiveComponent interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoSkinnedMeshComponentBase interface
	virtual FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FSkinnedMeshSceneProxyDesc& GetSkinnedMeshSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual FPrimitiveSceneProxy* AllocateSceneProxy() override;
	//~ End FFastGeoSkinnedMeshComponentBase interface

private:
	friend class FSkinnedMeshComponentHelper;

	FSkinnedMeshSceneProxyDesc SceneProxyDesc{};
};