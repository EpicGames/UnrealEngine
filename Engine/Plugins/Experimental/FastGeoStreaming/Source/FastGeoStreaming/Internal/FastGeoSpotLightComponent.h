// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoPointLightComponent.h"
#include "SpotLightSceneProxyDesc.h"

class FASTGEOSTREAMING_API FFastGeoSpotLightComponent : public FFastGeoPointLightComponentBase
{
public:
	typedef FFastGeoPointLightComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoSpotLightComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoSpotLightComponent() = default;

	//~ Begin FFastGeoComponent interface
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

protected:
	//~ Begin FFastGeoLightComponent interface
	virtual FLightSceneProxy* CreateTypedSceneProxy() override;
	//~ End FFastGeoLightComponent interface

	//~ Begin FFastGeoPointLightComponentBase interface
	virtual FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPointLightSceneProxyDesc& GetPointLightSceneProxyDesc() const override { return SceneProxyDesc; }
	//~ End FFastGeoPointLightComponentBase interface

private:
	FSpotLightSceneProxyDesc SceneProxyDesc{};
};
