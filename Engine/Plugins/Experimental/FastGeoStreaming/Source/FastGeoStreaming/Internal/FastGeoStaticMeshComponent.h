// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoMeshComponent.h"
#include "StaticMeshSceneProxyDesc.h"

class UBodySetup;
class UMaterialInterface;
class UNavCollisionBase;
class FVertexFactoryType;
struct FMaterialRelevance;
struct FStaticMeshLODResources;
struct FPrimitiveMaterialPropertyDescriptor;
namespace ERHIFeatureLevel { enum Type : int; }
namespace Nanite
{
	struct FResources;
	class FNaniteResourcesHelper;
}

class FASTGEOSTREAMING_API FFastGeoStaticMeshComponentBase : public FFastGeoMeshComponent, public IStaticMeshComponent
{
public:
	typedef FFastGeoMeshComponent Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoStaticMeshComponentBase(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoStaticMeshComponentBase() = default;

	const FCollisionResponseContainer& GetCollisionResponseToChannels() const;
	UMaterialInterface* GetNaniteAuditMaterial(int32 MaterialIndex) const;
	const Nanite::FResources* GetNaniteResources() const;
	bool HasValidNaniteData() const;
	bool UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const;
	bool ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials) const;
	virtual UObject const* AdditionalStatObject() const override;
	bool IsReverseCulling() const;
	bool IsDisallowNanite() const;
	bool IsForceDisableNanite() const;
	bool IsForceNaniteForMasked() const;
	int32 GetForcedLodModel() const;
	bool GetOverrideMinLOD() const;
	int32 GetMinLOD() const;
	bool GetForceNaniteForMasked() const;
	int32 GetWorldPositionOffsetDisableDistance() const;
	float GetNanitePixelProgrammableDistance() const;
	bool EvaluateWorldPositionOffsetInRayTracing() const;
#if WITH_EDITORONLY_DATA
	bool IsDisplayNaniteFallbackMesh() const;
#endif

	//~ Begin FFastGeoComponent interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void InitializeDynamicProperties() override;
	virtual UBodySetup* GetBodySetup() const override;
#if WITH_EDITOR
	virtual void InitializeFromComponent(UActorComponent* Component) override;
#endif
	//~ End FFastGeoComponent interface

	//~ Begin IStaticMeshComponent interface
#if WITH_EDITOR
	virtual void OnMeshRebuild(bool bRenderDataChanged) override {}
	virtual void PreStaticMeshCompilation() override {}
	virtual void PostStaticMeshCompilation() override {}
#endif
	virtual UStaticMesh* GetStaticMesh() const override;
	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() override;
	//~ End IStaticMeshComponent interface

	//~ Begin FFastGeoPrimitiveComponent interface
	virtual bool IsNavigationRelevant() const override;
	virtual FBox GetNavigationBounds() const override;
	virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	virtual void ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const override;
	virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	virtual int32 GetNumMaterials() const override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
#if WITH_EDITOR
	virtual void NotifyRenderStateChanged() override;
#endif
	//~ End FFastGeoPrimitiveComponent interface

protected:
	//~ Begin FFastGeoPrimitiveComponent interface
#if WITH_EDITOR
	virtual void InitializeSceneProxyDescFromComponent(UActorComponent* Component) override;
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
#endif
	virtual void InitializeSceneProxyDescDynamicProperties() override;
	virtual void ApplyWorldTransform(const FTransform& InTransform) override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy(ESceneProxyCreationError* OutError = nullptr) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoMeshComponent interface
	virtual UMaterialInterface* GetOverlayMaterial() const override;
	virtual const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const override;
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const override;
	//~ End FFastGeoMeshComponent interface

	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() = 0;
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const = 0;
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite);
	UMaterialInterface* GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit) const;
	bool ShouldExportAsObstacle(const UNavCollisionBase& InNavCollision) const;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams);
#endif

protected:
	bool bUseDefaultCollision = false;
	// Temporary until LODData support
	struct FFastGeoStaticMeshComponentLODInfo
	{
		FColorVertexBuffer* OverrideVertexColors;
	};
	TArray<FFastGeoStaticMeshComponentLODInfo> LODData;
	FDelegateHandle OnResourceUpdated;
	FDelegateHandle OnContainerUnregistered;

private:
	friend class FMeshComponentHelper;
	friend class FPrimitiveComponentHelper;
	friend class FStaticMeshComponentHelper;
	friend class FInstancedStaticMeshComponentHelper;
	friend class Nanite::FNaniteResourcesHelper;
};

class FASTGEOSTREAMING_API FFastGeoStaticMeshComponent : public FFastGeoStaticMeshComponentBase
{
public:
	typedef FFastGeoStaticMeshComponentBase Super;

	/** Static type identifier for this element class */
	static const FFastGeoElementType Type;

	FFastGeoStaticMeshComponent(int32 InComponentIndex = INDEX_NONE, FFastGeoElementType InType = Type);
	virtual ~FFastGeoStaticMeshComponent() = default;

protected:
	//~ Begin FFastGeoPrimitiveComponent interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override { return SceneProxyDesc; }
	virtual void GetBodyInstances(TArray<FBodyInstance*>& OutBodyInstances) override;
	//~ End FFastGeoPrimitiveComponent interface

	//~ Begin FFastGeoStaticMeshComponentBase interface
	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() override { return SceneProxyDesc; }
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const override { return SceneProxyDesc; }
	//~ End FFastGeoStaticMeshComponentBase interface

private:
	FStaticMeshSceneProxyDesc SceneProxyDesc{};
};