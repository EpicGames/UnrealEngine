// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "FastGeoSurrogateActor.generated.h"

class UFastGeoSurrogateComponent;
struct FFastGeoSurrogateComponentDescriptor;

UCLASS(NotBlueprintable)
class FASTGEOSTREAMING_API AFastGeoSurrogateActor : public AActor
{
	GENERATED_BODY()

public:

	AFastGeoSurrogateActor();

	const FFastGeoSurrogateComponentDescriptor* GetSurrogateComponentDescriptor(int32 SurrogateComponentDescriptorIndex) const;
	UFastGeoSurrogateComponent* GetSurrogateComponent(int32 SurrogateComponentDescriptorIndex) const;
	bool HasRegisteredWithFastGeo() const { return bHasRegisteredWithFastGeo; }
	bool IsActive() const { return bIsActive; }

protected:

	//~ Begin AActor interface
	virtual void PostInitializeComponents() override;
	virtual void PostUnregisterAllComponents() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End AActor interface

private:

	bool UnregisterFromFastGeo();

	/** Maps surrogate component descriptor index to surrogate component descriptor */
	UPROPERTY()
	TArray<FFastGeoSurrogateComponentDescriptor> SurrogateComponentDescriptors;

	/** Maps surrogate component descriptor index to surrogate component */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UFastGeoSurrogateComponent>> SurrogateComponents;

	/** Whether actor has registered with FastGeo container */
	UPROPERTY(Transient)
	bool bHasRegisteredWithFastGeo = false;

	UPROPERTY(Transient)
	bool bIsActive = false;

	friend class UFastGeoWorldPartitionRuntimeCellTransformer;
};