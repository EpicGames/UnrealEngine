// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FFastGeoSurrogateBodyInstanceIndex
{
	static FFastGeoSurrogateBodyInstanceIndex FromEncoded(int32 InEncodedIndex)
	{
		check(InEncodedIndex == INDEX_NONE || IsEncoded(InEncodedIndex));
		return FFastGeoSurrogateBodyInstanceIndex(InEncodedIndex);
	}

	static FFastGeoSurrogateBodyInstanceIndex Encode(int32 InRawIndex)
	{
		check(InRawIndex == INDEX_NONE || InRawIndex >= 0);
		check(InRawIndex == INDEX_NONE || ((InRawIndex & BodyInstanceIndexEncodeBit) == 0));
		return (InRawIndex == INDEX_NONE)
			? FFastGeoSurrogateBodyInstanceIndex(INDEX_NONE)
			: FFastGeoSurrogateBodyInstanceIndex(InRawIndex | BodyInstanceIndexEncodeBit);
	}

	static int32 Decode(int32 Index)
	{
		return (Index == INDEX_NONE) ? INDEX_NONE : (Index & ~BodyInstanceIndexEncodeBit);
	}

	static bool IsEncoded(int32 Index)
	{
		return (Index != INDEX_NONE) && (Index >= 0) && ((Index & BodyInstanceIndexEncodeBit) != 0);
	}

	int32 Decode() const
	{
		return Decode(EncodedIndex);
	}

	bool IsEncoded() const
	{
		return IsEncoded(EncodedIndex);
	}

	int32 GetEncoded() const
	{
		return EncodedIndex;
	}

	explicit operator int32() const
	{
		return EncodedIndex;
	}

	FFastGeoSurrogateBodyInstanceIndex& operator++()
	{
		return (*this += 1);
	}

	FFastGeoSurrogateBodyInstanceIndex operator++(int)
	{
		FFastGeoSurrogateBodyInstanceIndex Temp = *this;
		++(*this);
		return Temp;
	}

	FFastGeoSurrogateBodyInstanceIndex& operator+=(int32 Count)
	{
		check(Count >= 0);
		check(EncodedIndex != INDEX_NONE);

		const int64 Raw = (int64)Decode();
		const int64 NewRaw = Raw + (int64)Count;

		check(NewRaw >= 0);
		check(NewRaw < (int64)BodyInstanceIndexEncodeBit);

		EncodedIndex = (int32)NewRaw | BodyInstanceIndexEncodeBit;
		return *this;
	}

	friend FFastGeoSurrogateBodyInstanceIndex operator+(FFastGeoSurrogateBodyInstanceIndex Lhs, int32 Count)
	{
		Lhs += Count;
		return Lhs;
	}

private:
	explicit FFastGeoSurrogateBodyInstanceIndex(int32 InEncoded)
		: EncodedIndex(InEncoded)
	{
		check(InEncoded == INDEX_NONE || IsEncoded(InEncoded));
	}

	int32 EncodedIndex = INDEX_NONE;

	/** Bit 30 used to flag that a BodyInstance Index has been encoded. */
	static constexpr int32 BodyInstanceIndexEncodeBit = 1 << 30;
};