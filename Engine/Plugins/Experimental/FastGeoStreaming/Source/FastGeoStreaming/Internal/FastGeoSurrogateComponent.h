// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/BodyInstance.h"
#include "FastGeoPrimitiveComponent.h"

#include "FastGeoSurrogateComponent.generated.h"

struct FFastGeoSurrogateComponentDescriptor;

UCLASS(NotBlueprintable)
class FASTGEOSTREAMING_API UFastGeoSurrogateComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:

	UFastGeoSurrogateComponent();

	//~ Begin UActorComponent interface
	virtual void OnRegister() override;
	virtual bool ShouldCreatePhysicsState() const override;
	virtual void OnCreatePhysicsState() override;
	virtual void OnDestroyPhysicsState() override;
	//~ End UActorComponent interface

	// Surrogates carry a placeholder BodyInstance only because UPrimitiveComponent has one; the real per-item BodyInstances are static and
	// owned by the FastGeo container. Override base getters that would read the placeholder to return static-correct constants instead.

	//~ Begin UPrimitiveComponent interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override { return nullptr; }
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	virtual bool IsGravityEnabled() const override { return false; }
	virtual bool GetUpdateKinematicFromSimulation() const override { return false; }
	virtual bool GetGyroscopicTorqueEnabled() const override { return false; }
	virtual bool ShouldDispatchWakeEvents(FName BoneName) const override { return false; }
	virtual float GetLinearDamping() const override { return 0.f; }
	virtual float GetAngularDamping() const override { return 0.f; }
	virtual FVector GetInertiaTensor(FName BoneName = NAME_None) const override { return FVector::ZeroVector; }
	virtual float GetMaxDepenetrationVelocity(FName BoneName = NAME_None) override { return -1.f; }
	virtual bool IsAnyRigidBodyAwake() override { return false; }
	//~ End UPrimitiveComponent interface

	//~ Begin USceneComponent interface
	virtual FVector GetComponentVelocity() const override { return FVector::ZeroVector; }
	virtual bool IsSimulatingPhysics(FName BoneName = NAME_None) const override { return false; }
	virtual ECollisionEnabled::Type GetCollisionEnabled() const override;
	virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const override;
	virtual const FCollisionResponseContainer& GetCollisionResponseToChannels() const override;
	//~ End USceneComponent interface

	//~ Begin IPhysicsBodyInstanceOwner interface
	virtual bool IsMultiBodyOverlap() const override { return true; }
	//~ End IPhysicsBodyInstanceOwner interface

	//~ Begin IPhysicsComponent interface
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	//~ End IPhysicsComponent interface

	void RegisterBodyInstances(const TArray<FBodyInstance*>& InBodyInstances);
	void UnregisterBodyInstances();

protected:

	bool HasRegisteredBodyInstances() const { return !IndexToBodyInstance.IsEmpty(); }
	void SetSurrogateComponentDescriptor(const FFastGeoSurrogateComponentDescriptor& InComponentDesc);
	int32 GetDescriptorIndex() const { return DescriptorIndex; }

	UPROPERTY()
	int32 DescriptorIndex = INDEX_NONE;

	TArray<FBodyInstance*> IndexToBodyInstance;

	friend class FFastGeoPhysicsBodyInstanceOwner;
	friend class AFastGeoSurrogateActor;
	friend class UFastGeoWorldPartitionRuntimeCellTransformer;
};