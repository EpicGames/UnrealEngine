// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "FastGeoSurrogateComponentDescriptor.generated.h"

USTRUCT()
struct FFastGeoSurrogateComponentDescriptor
{
	GENERATED_BODY()

	UPROPERTY()
	TEnumAsByte<ECollisionEnabled::Type> CollisionEnabled;

	UPROPERTY()
	TEnumAsByte<ECollisionChannel> ObjectType;

	UPROPERTY()
	FCollisionResponseContainer ResponseContainer;

	UPROPERTY()
	FWalkableSlopeOverride WalkableSlopeOverride;

	UPROPERTY()
	TEnumAsByte<ECanBeCharacterBase> CanCharacterStepUpOn;

	FFastGeoSurrogateComponentDescriptor()
		: CollisionEnabled(ECollisionEnabled::NoCollision)
		, ObjectType(ECollisionChannel::ECC_WorldStatic)
		, CanCharacterStepUpOn(ECanBeCharacterBase::ECB_No)
	{
	}

#if WITH_EDITOR
	FFastGeoSurrogateComponentDescriptor(const UPrimitiveComponent& InPrimitiveComponent)
		: CollisionEnabled(InPrimitiveComponent.BodyInstance.GetCollisionEnabled())
		, ObjectType(InPrimitiveComponent.BodyInstance.GetObjectType())
		, ResponseContainer(InPrimitiveComponent.BodyInstance.GetResponseToChannels())
		, WalkableSlopeOverride(ResolveWalkableSlopeOverride(InPrimitiveComponent))
		, CanCharacterStepUpOn(InPrimitiveComponent.CanCharacterStepUpOn)
	{
	}

private:
	// FBodyInstance::GetWalkableSlopeOverride() falls back to the BodySetup's value only when
	// FBodyInstance::BodySetup is valid -- which is only true after InitBody() has run. The
	// surrogate descriptor is built at cell-transform time before any per-actor physics state
	// creation, so the BodyInstance read short-circuits to its (default-empty) member and the
	// BodySetup default authored on the static mesh asset is lost. Resolve it explicitly here,
	// mirroring what FBodyInstance::GetWalkableSlopeOverride() would return after init.
	static FWalkableSlopeOverride ResolveWalkableSlopeOverride(const UPrimitiveComponent& InPrimitiveComponent)
	{
		const FBodyInstance& SrcBI = InPrimitiveComponent.BodyInstance;
		if (SrcBI.GetOverrideWalkableSlopeOnInstance())
		{
			return SrcBI.GetWalkableSlopeOverride();
		}
		if (const UBodySetup* BodySetup = const_cast<UPrimitiveComponent&>(InPrimitiveComponent).GetBodySetup())
		{
			return BodySetup->WalkableSlopeOverride;
		}
		return FWalkableSlopeOverride();
	}

public:
#endif

	FORCEINLINE bool operator==(const FFastGeoSurrogateComponentDescriptor& Other) const
	{
		return CollisionEnabled == Other.CollisionEnabled &&
			ObjectType == Other.ObjectType &&
			ResponseContainer == Other.ResponseContainer &&
			WalkableSlopeOverride.WalkableSlopeBehavior == Other.WalkableSlopeOverride.WalkableSlopeBehavior &&
			WalkableSlopeOverride.WalkableSlopeAngle == Other.WalkableSlopeOverride.WalkableSlopeAngle &&
			CanCharacterStepUpOn == Other.CanCharacterStepUpOn;
	}
};