// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoContainer.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoComponent.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FWeakFastGeoComponentCluster
{
public:
	FWeakFastGeoComponentCluster(FFastGeoComponentCluster* InComponentCluster = nullptr)
	{
		if (UFastGeoContainer* Container = InComponentCluster ? InComponentCluster->GetOwnerContainer() : nullptr)
		{
			ContainerWeak = Container;
			ComponentClusterTypeID = InComponentCluster->GetTypeID();
			ComponentClusterIndex = InComponentCluster->GetComponentClusterIndex();
		}
	}

	FORCEINLINE bool operator==(const FWeakFastGeoComponentCluster& Other) const
	{
		return ContainerWeak == Other.ContainerWeak &&
			ComponentClusterTypeID == Other.ComponentClusterTypeID &&
			ComponentClusterIndex == Other.ComponentClusterIndex;
	}

	FORCEINLINE FFastGeoComponentCluster* Get() const
	{
		if (UFastGeoContainer* Container = ContainerWeak.Get())
		{
			return Container->GetComponentCluster(ComponentClusterTypeID, ComponentClusterIndex);
		}
		return nullptr;
	}

	template <class T>
	FORCEINLINE T* Get() const
	{
		FFastGeoComponentCluster* ComponentCluster = Get();
		return ComponentCluster ? ComponentCluster->CastTo<T>() : nullptr;
	}

private:
	TWeakObjectPtr<UFastGeoContainer> ContainerWeak;
	uint32 ComponentClusterTypeID = INDEX_NONE;
	int32 ComponentClusterIndex = INDEX_NONE;
};

class FWeakFastGeoComponent
{
public:
	FWeakFastGeoComponent() = default;

	FWeakFastGeoComponent(FFastGeoComponent* InComponent)
	{
		if (FFastGeoComponentCluster* ComponentCluster = InComponent ? InComponent->GetOwnerComponentCluster() : nullptr)
		{
			ComponentClusterWeak = ComponentCluster;
			ComponentTypeID = InComponent->GetTypeID();
			ComponentIndex = InComponent->GetComponentIndex();
		}
		else
		{
			ensure(!InComponent);
		}
	}

	FORCEINLINE bool operator==(const FWeakFastGeoComponent& Other) const
	{
		return ComponentClusterWeak == Other.ComponentClusterWeak &&
			ComponentTypeID == Other.ComponentTypeID &&
			ComponentIndex == Other.ComponentIndex;
	}

	FORCEINLINE FFastGeoComponent* Get() const
	{
		if (FFastGeoComponentCluster* ComponentCluster = ComponentClusterWeak.Get())
		{
			return ComponentCluster->GetComponent(ComponentTypeID, ComponentIndex);
		}
		return nullptr;
	}

	template <class T>
	FORCEINLINE T* Get() const
	{
		FFastGeoComponent* Component = Get();
		return Component ? Component->CastTo<T>() : nullptr;
	}

private:
	FWeakFastGeoComponentCluster ComponentClusterWeak;
	uint32 ComponentTypeID = INDEX_NONE;
	int32 ComponentIndex = INDEX_NONE;
};
