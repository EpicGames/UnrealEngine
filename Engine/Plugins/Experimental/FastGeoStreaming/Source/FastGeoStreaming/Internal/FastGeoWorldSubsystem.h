// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "FastGeoWeakElement.h"
#include "FastGeoRegisteredComponent.h"
#include "FastGeoAsyncRenderStateJobQueue.h"
#include "FastGeoWorldSubsystem.generated.h"

class FFastGeoPrimitiveComponent;
class ULevelStreaming;
class ULevel;
class IWorldPartitionHLODObject;
enum class ELevelStreamingState : uint8;

UCLASS()
class FASTGEOSTREAMING_API UFastGeoWorldSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

	UFastGeoWorldSubsystem();
	~UFastGeoWorldSubsystem();

public:
	//~Begin USubsystem interface
	virtual void Deinitialize() override;
	//~End USubsystem interface

	//~ Begin UWorldSubsystem interface.
	virtual void PostInitialize() override;
	virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	//~ End UWorldSubsystem interface.

	//~ Begin UTickableWorldSubsystem interface
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End UTickableWorldSubsystem interface

	void AddToComponentsPendingRecreate(FFastGeoComponent* InComponent);
	void ProcessPendingRecreate();

	/**
	 * Start a job that will precache PSOs asynchronously for a specified FastGeo container.
	 * @param FastGeo: FastGeo container.
	 */
	void PushAsyncPrecachePSOsJob(UFastGeoContainer* FastGeo);

	/**
	 * Start a job that will create render state asynchronously for a specified FastGeo container.
	 * @param FastGeo: FastGeo container.
	 */
	void PushAsyncCreateRenderStateJob(UFastGeoContainer* FastGeo);

	/**
	 * Start a job that will destroy render state asynchronously for a specified FastGeo container.
	 * @param FastGeo: FastGeo container.
	 */
	void PushAsyncDestroyRenderStateJob(UFastGeoContainer* FastGeo);

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	/** Start a job to create render state for PSO-deferred components (DelayUntilPSOPrecached). */
	void PushAsyncDeferredCreateJob(UFastGeoContainer* FastGeo);

	/** Start a job to recreate render state for components with fallback material (UseFallbackMaterialUntilPSOPrecached). */
	void PushAsyncRecreateRenderStateJob(UFastGeoContainer* FastGeo);
#endif

	/**
	 * Update progress of asynchronous render state creation and destruction.
	 * @param bWaitForCompletion : Whether to wait for pending asynchronous tasks to complete.
	 */
	void ProcessAsyncRenderStateJobs(bool bWaitForCompletion = false);

	/**
	 * Start a job that will create physics state asynchronously for a specified FastGeo container.
	 * @param FastGeo: FastGeo container.
	 */
	void PushAsyncCreatePhysicsStateJobs(UFastGeoContainer* FastGeo);

	/**
	 * Start a job that will destroy physics state asynchronously for a specified FastGeo container.
	 * @param FastGeo: FastGeo container.
	 */
	void PushAsyncDestroyPhysicsStateJobs(UFastGeoContainer* FastGeo);

	static bool IsEnableDebugView();
	static bool ShouldAllowSurrogateComponents();

#if !UE_BUILD_SHIPPING
	static bool IsFastGeoVisible();
#endif

	bool IsWaitingForCompletion() const;
	bool IsReregistering() const { return bIsReregistering; }

#if WITH_EDITOR
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnComponentsPreRecreate, const TArray<FFastGeoRegisteredComponent>&);

	/** Called right before processing any registered FastGeo components pending recreate each frame. */
	static FOnComponentsPreRecreate ComponentsPreRecreateEvent;
#endif

private:
#if !UE_BUILD_SHIPPING
	void TickDebugTraceHUD();
#endif

	void OnUpdateLevelStreaming();
	void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	void OnLevelStreamingStateChanged(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState PrevState, ELevelStreamingState NewState);
	void OnLevelStartedAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* Level);
	void OnLevelStartedRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* Level);
	void OnLevelComponentsUpdated(UWorld* World, ULevel* Level);
	void OnLevelComponentsCleared(UWorld* World, ULevel* Level);
	void OnAddLevelToWorldExtension(ULevel* Level, const bool bWaitForCompletion, bool& bOutHasCompleted);
	void OnRemoveLevelFromWorldExtension(ULevel* Level, const bool bWaitForCompletion, bool& bOutHasCompleted);
	void OnGlobalComponentReregisterContextCreated();
	void OnGlobalComponentReregisterContextDestroyed();
#if DO_CHECK
	void OnLevelAddedToWorld(ULevel* Level, UWorld* World);
	void OnLevelRemovedFromWorld(ULevel* Level, UWorld* World);
	void CheckNoPendingTasks(ULevel* Level, UWorld* World, bool bIncludeRecreateTasks = true);
#endif
#if WITH_EDITOR
	void OnPreRecreateScene(UWorld* World);
#endif
	void ForEachHLODObjectInCell(const UWorldPartitionRuntimeCell*, TFunction<void(IWorldPartitionHLODObject*)>);
	void RequestAsyncRenderStateTasksBudget_Concurrent(float& OutAvailableTimeBudgetMS, int32& OutAvaiableComponentsBudget, int32& OutTimeEpoch);
	void CommitAsyncRenderStateTasksBudget_Concurrent(float InUsedTimeBudgetMS, int32& InUsedComponentsBudget, int32 TimeEpoch);
	FFastGeoAsyncRenderStateJobQueue* GetOrCreateJobQueue();
	void WaitForAllPendingWorkCompletion();
	void PushRuntimeContainerForRegistration(UFastGeoContainer* FastGeo);
	void PushRuntimeContainerForDestruction(UFastGeoContainer* FastGeo);
	void DrainPendingRuntimeContainers();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	void RegisterContainerWithPSODeferredWork(UFastGeoContainer* FastGeo);
	void UnregisterContainerWithPSODeferredWork(UFastGeoContainer* FastGeo);
	void RegisterContainerWithAsyncPSORecreateWork(UFastGeoContainer* FastGeo);
	void UnregisterContainerWithAsyncPSORecreateWork(UFastGeoContainer* FastGeo);
#endif

	UPROPERTY()
	TSet<TObjectPtr<UFastGeoContainer>> PendingRuntimeRegisterContainers;

	UPROPERTY()
	TSet<TObjectPtr<UFastGeoContainer>> PendingRuntimeDestroyContainers;

	FDelegateHandle Handle_OnLevelStreamingStateChanged;
	FDelegateHandle Handle_OnLevelBeginAddToWorld;
	FDelegateHandle Handle_OnLevelBeginRemoveFromWorld;
	FDelegateHandle Handle_OnForEachHLODObjectInCell;

	// Manages components pending render state recreation (e.g., MarkRenderStateDirty from runtime
	// property changes, decal fade, or PSO fallback material swap in sync mode).
	// Non-PSO recreates are processed immediately (same frame guarantee).
	// PSO recreates (NeedsPSORecreate) are time-sliced with a per-frame budget.
	struct FFastGeoPendingRecreateManager
	{
		void Add(FFastGeoComponent* Component);
		void Process(UWorld* World);
		bool IsEmpty() const;
		void Reset();

#if WITH_EDITOR
		const TArray<FFastGeoRegisteredComponent>& GetComponents() const { return Components; }
#endif

	private:
		TArray<FFastGeoRegisteredComponent> Components;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		TFastGeoRenderStateBatch<FFastGeoRegisteredComponent> PSORecreateBatch;
#endif
		bool bHasNonPSORecreate = false;
	} PendingRecreateManager;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Containers with PSO-deferred components waiting for PSO callbacks.
	// Polled each Tick() to detect lost callbacks (component destroyed, world torn down, etc.)
	// and unblock registration that would otherwise hang forever.
	TSet<TWeakObjectPtr<UFastGeoContainer>> ContainersWithPSODeferredWork;

	// Containers with pending PSO recreate work (UseFallbackMaterialUntilPSOPrecached).
	// Tracked so Tick() can drive ProcessAsyncRenderStateJobs for recreate jobs that were
	// pushed after OnAddLevelToWorldExtension completed. See Tick() comment for details.
	TSet<TWeakObjectPtr<UFastGeoContainer>> ContainersWithAsyncPSORecreateWork;
#endif

	static bool bEnableDebugView;

	FRWLock Lock;
	int32 TimeEpoch = 0;
	float UsedAsyncRenderStateTasksTimeBudgetMS = 0;
	int32 UsedNumComponentsToProcessBudget = 0;
	// Stamped on GT when entering a blocking wait window (level add/remove with bWaitForCompletion,
	// global reregister, full drain). Read from worker threads via IsWaitingForCompletion() to make
	// PSO-skip decisions during proxy creation.
	std::atomic<bool> bWaitingForCompletion{ false };
	bool bIsReregistering = false;

	TUniquePtr<FFastGeoAsyncRenderStateJobQueue> AsyncRenderStateJobQueue;
	TArray<TWeakObjectPtr<UFastGeoContainer>> ReregisteringFastGeoContainers;

	friend class UFastGeoContainer;
};
