// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastGeoElementType.h"

class IFastGeoElement
{
public:
	/** Default constructor */
	IFastGeoElement(FFastGeoElementType InType) : ElementType(InType) {}
	virtual ~IFastGeoElement() = default;

	/** Attempt to cast this element to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	T* CastTo()
	{
		return ElementType.IsA(T::Type) ? StaticCast<T*>(this) : nullptr;
	}
	/** Attempt to cast this element to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	const T* CastTo() const
	{
		return ElementType.IsA(T::Type) ? StaticCast<const T*>(this) : nullptr;
	}
	/** Cast this element to another type and return a reference. Will assert if the cast fails. */
	template <typename T>
	T& CastToRef()
	{
		check(ElementType.IsA(T::Type));
		return *StaticCast<T*>(this);
	}
	/** Cast this element to another type and return a reference. Will assert if the cast fails. */
	template <typename T>
	const T& CastToRef() const
	{
		check(ElementType.IsA(T::Type));
		return *StaticCast<const T*>(this);
	}
	/** Returns true if this element is of the specified type */
	template <typename T>
	bool IsA() const
	{
		return ElementType.IsA(T::Type);
	}

	/** Returns the element type unique ID */
	uint32 GetTypeID() { return ElementType.ID; }

protected:
	/** Static type identifier for the base class */
	static const FFastGeoElementType Type;

	/** Element type identifier */
	FFastGeoElementType ElementType;
};

