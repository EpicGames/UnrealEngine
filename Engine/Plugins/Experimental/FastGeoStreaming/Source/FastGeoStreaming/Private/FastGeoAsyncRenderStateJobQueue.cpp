// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoAsyncRenderStateJobQueue.h"
#include "FastGeoContainer.h"
#include "FastGeoWorldSubsystem.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "SceneInterface.h"

FFastGeoAsyncRenderStateJobQueue::FFastGeoAsyncRenderStateJobQueue(FSceneInterface* InScene)
	: Scene(InScene)
{
	OnAllLevelsChangedHandle = Scene->GetWorld()->OnAllLevelsChanged().AddLambda([this]() { OnUpdateLevelStreamingDone(); });
}

FFastGeoAsyncRenderStateJobQueue::~FFastGeoAsyncRenderStateJobQueue()
{
	Scene->GetWorld()->OnAllLevelsChanged().Remove(OnAllLevelsChangedHandle);
	Launch();
	WaitForAsyncTasksExecution();
	OnAsyncTasksExecuted();
	check(IsCompleted());
}

void FFastGeoAsyncRenderStateJobQueue::AddJob(const FJob& Job)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::AddJob);
	check(IsInGameThread());

	if (!PendingJobs.IsValid())
	{
		PendingJobs = MakeUnique<FJobSet>();
	}
	PendingJobs->Add(Job);

	check(::IsValid(Job.FastGeo));
	{
		switch (Job.Type)
		{
		case EJobType::PrecachePSOs:
			Job.FastGeo->OnPrecachePSOsBegin_GameThread();
			break;
		case EJobType::CreateRenderState:
			Job.FastGeo->OnCreateRenderStateBegin_GameThread();
			break;
		case EJobType::DestroyRenderState:
			Job.FastGeo->OnDestroyRenderStateBegin_GameThread();
			break;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		case EJobType::DeferredCreateRenderState:
			Job.FastGeo->OnDeferredCreateRenderStateBegin_GameThread();
			break;
		case EJobType::RecreateRenderState:
			Job.FastGeo->OnRecreateRenderStateBegin_GameThread();
			break;
#endif
		}
	}
}

void FFastGeoAsyncRenderStateJobQueue::OnAsyncTasksExecuted()
{
	check(AreAsyncTasksExecuted());
	for (const FJobs& Jobs : PipedJobs)
	{
		Jobs->OnPostExecute_GameThread();
	}
	PipedJobs.Reset();
	PipeTasks.Reset();
	IsReadyToRunAsyncTasksEvent.Reset();
}

bool FFastGeoAsyncRenderStateJobQueue::AreAsyncTasksExecuted() const
{
	return !PendingJobs.IsValid() && !Pipe.HasWork();
}

bool FFastGeoAsyncRenderStateJobQueue::IsCompleted() const
{
	return AreAsyncTasksExecuted() && PipedJobs.IsEmpty() && PipeTasks.IsEmpty();
}

void FFastGeoAsyncRenderStateJobQueue::Launch()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::Launch);
	check(IsInGameThread());

	if (!PendingJobs.IsValid() || PendingJobs->IsEmpty())
	{
		return;
	}

	// Move Jobs to the async queue
	FJobs& NewJobs = PipedJobs.Emplace_GetRef();
	NewJobs = MoveTemp(PendingJobs);
	if (!IsReadyToRunAsyncTasksEvent)
	{
		IsReadyToRunAsyncTasksEvent.Emplace(UE_SOURCE_LOCATION);
	}
	// Launch a new task in the pipe
	PipeTasks.Emplace(Pipe.Launch(UE_SOURCE_LOCATION, [this, NewJobs = NewJobs.Get()]() { NewJobs->Execute(); }, *IsReadyToRunAsyncTasksEvent, UE::Tasks::ETaskPriority::BackgroundHigh));
}

void FFastGeoAsyncRenderStateJobQueue::OnUpdateLevelStreamingDone()
{
	TriggerIsReadyToRunAsyncTasksEvent();
}

void FFastGeoAsyncRenderStateJobQueue::TriggerIsReadyToRunAsyncTasksEvent()
{
	if (IsReadyToRunAsyncTasksEvent)
	{
		IsReadyToRunAsyncTasksEvent->Trigger();
		IsReadyToRunAsyncTasksEvent.Reset();
	}
}

void FFastGeoAsyncRenderStateJobQueue::WaitForAsyncTasksExecution()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::WaitForAsyncTasksExecution);
	check(IsInGameThread());

	TriggerIsReadyToRunAsyncTasksEvent();
	UE::Tasks::Wait(PipeTasks);
}

void FFastGeoAsyncRenderStateJobQueue::Tick(bool bForceWaitCompletion)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::Tick);
	check(IsInGameThread());

	// Flush all queued jobs
	Launch();

#if WITH_EDITOR
	// In non-game worlds make sure event is triggered as OnUpdateLevelStreamingDone isn't called
	if (!Scene->GetWorld()->IsGameWorld())
	{
		TriggerIsReadyToRunAsyncTasksEvent();
	}
#endif

	// Wait for tasks if the world is inside a blocking load
	if (bForceWaitCompletion || Scene->GetWorld()->GetSubsystem<UFastGeoWorldSubsystem>()->IsWaitingForCompletion())
	{
		WaitForAsyncTasksExecution();
	}

	// Process completed tasks
	check(PipeTasks.Num() == PipedJobs.Num());
	for (int i = 0; i < PipeTasks.Num();)
	{
		if (PipeTasks[i].IsCompleted())
		{
			PipedJobs[i]->OnPostExecute_GameThread();
			PipeTasks.RemoveAt(i, EAllowShrinking::No);
			PipedJobs.RemoveAt(i, EAllowShrinking::No);
		}
		else
		{
			++i;
		}
	}
}

void FFastGeoAsyncRenderStateJobQueue::FJobSet::Execute() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::ExecuteJobs);
	for (const FJob& Job : Jobs)
	{
		Job.Execute();
	}
}

void FFastGeoAsyncRenderStateJobQueue::FJobSet::OnPostExecute_GameThread() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::OnPostExecute_GameThread);
	for (const FJob& Job : Jobs)
	{
		Job.OnPostExecute_GameThread();
	}
}

void FFastGeoAsyncRenderStateJobQueue::FJob::Execute() const
{
	if (::IsValid(FastGeo))
	{
		switch (Type)
		{
		case EJobType::PrecachePSOs:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::PrecachePSOs);
			FastGeo->OnPrecachePSOs_Concurrent();
		}
		break;
		case EJobType::CreateRenderState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::CreateRenderState);
			FastGeo->OnCreateRenderState_Concurrent();
		}
		break;
		case EJobType::DestroyRenderState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::DestroyRenderState);
			FastGeo->OnDestroyRenderState_Concurrent();
		}
		break;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		case EJobType::DeferredCreateRenderState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::DeferredCreateRenderState);
			FastGeo->OnDeferredCreateRenderState_Concurrent();
		}
		break;
		case EJobType::RecreateRenderState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoAsyncRenderStateJobQueue::RecreateRenderState);
			FastGeo->OnRecreateRenderState_Concurrent();
		}
		break;
#endif
		}
	}
}

void FFastGeoAsyncRenderStateJobQueue::FJob::OnPostExecute_GameThread() const
{
	if (::IsValid(FastGeo))
	{
		switch (Type)
		{
		case EJobType::PrecachePSOs:
			FastGeo->OnPrecachePSOsEnd_GameThread();
			break;
		case EJobType::CreateRenderState:
			FastGeo->OnCreateRenderStateEnd_GameThread();
			break;
		case EJobType::DestroyRenderState:
			FastGeo->OnDestroyRenderStateEnd_GameThread();
			break;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		case EJobType::DeferredCreateRenderState:
			FastGeo->OnDeferredCreateRenderStateEnd_GameThread();
			break;
		case EJobType::RecreateRenderState:
			FastGeo->OnRecreateRenderStateEnd_GameThread();
			break;
#endif
		}
	}
}