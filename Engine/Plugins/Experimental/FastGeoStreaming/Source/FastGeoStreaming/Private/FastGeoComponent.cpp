// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoComponent.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoContainer.h"
#include "FastGeoWorldSubsystem.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "UnrealEngine.h"

const FFastGeoElementType FFastGeoComponent::Type(&IFastGeoElement::Type);

FFastGeoComponent::FFastGeoComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InType)
	, ComponentIndex(InComponentIndex)
	, Owner(nullptr)
	, Lock(MakeUnique<FRWLock>())
{
}

FFastGeoComponent::FFastGeoComponent(const FFastGeoComponent& Other)
: Super(Other)
, ComponentIndex(Other.ComponentIndex)
, LocalTransform(Other.LocalTransform)
, WorldTransform(Other.WorldTransform)
, DetailMode(Other.DetailMode)
, Owner(Other.Owner)
, Lock(MakeUnique<FRWLock>())
#ifdef FASTGEO_DEBUG_PHYSICS
, PhysicsStateCreation(EPhysicsStateCreation::NotCreated)
#endif
, ProxyState(EProxyCreationState::None)
, bRenderStateDirty(false)
{
}

#if WITH_EDITOR
void FFastGeoComponent::InitializeFromComponent(UActorComponent* InComponent)
{
	USceneComponent* SceneComponent = CastChecked<USceneComponent>(InComponent);

	// Refresh ComponentToWorld root-first along the attach chain before reading the leaf. Calling UpdateComponentToWorld on the leaf
	// alone is unsafe: a Blueprint scene component ancestor can be born with bComponentToWorldUpdated already true while its
	// ComponentToWorld is still identity, because the flag is UPROPERTY(Transient) and copies from the archetype on instance
	// construction whereas ComponentToWorld is a plain field that defaults to identity. UpdateComponentToWorld then skips the
	// parent recursion and reads the stale parent transform. Refreshing ancestors first lets each link recompute through
	// CalcNewComponentToWorld, preserving socket attachments and absolute-location/rotation/scale flags.
	TArray<USceneComponent*, TInlineAllocator<8>> AttachChain;
	for (USceneComponent* ChainComponent = SceneComponent; ChainComponent != nullptr; ChainComponent = ChainComponent->GetAttachParent())
	{
		AttachChain.Add(ChainComponent);
	}
	for (int32 ChainIndex = AttachChain.Num() - 1; ChainIndex >= 0; --ChainIndex)
	{
		AttachChain[ChainIndex]->UpdateComponentToWorld();
	}

	LocalTransform = SceneComponent->GetComponentToWorld();
	WorldTransform = LocalTransform;
	DetailMode = SceneComponent->DetailMode;
}
#endif

FArchive& operator<<(FArchive& Ar, FFastGeoComponent& Component)
{
	Component.Serialize(Ar);
	return Ar;
}

void FFastGeoComponent::SetOwnerComponentCluster(FFastGeoComponentCluster* InOwner)
{
	check(InOwner);
	Owner = InOwner;
}

FFastGeoComponentCluster* FFastGeoComponent::GetOwnerComponentCluster() const
{
	return Owner;
}

UFastGeoContainer* FFastGeoComponent::GetOwnerContainer() const
{
	check(GetOwnerComponentCluster());
	return GetOwnerComponentCluster()->GetOwnerContainer();
}

UWorld* FFastGeoComponent::GetWorld() const
{
	return GetOwnerContainer()->GetWorld();
}

FSceneInterface* FFastGeoComponent::GetScene() const
{
	return GetOwnerContainer()->GetScene();
}

FPhysScene* FFastGeoComponent::GetPhysicsScene() const
{
	return GetOwnerContainer()->GetPhysicsScene();
}

float FFastGeoComponent::GetWorldTimeSeconds() const
{
	return GetOwnerContainer()->GetWorldTimeSeconds();
}

FLinearColor FFastGeoComponent::GetDebugColor() const
{
	return UFastGeoWorldSubsystem::IsEnableDebugView() ? FLinearColor::Blue : FLinearColor::White;
}

void FFastGeoComponent::Serialize(FArchive& Ar)
{
	Ar << ComponentIndex;
	Ar << LocalTransform;
	Ar << WorldTransform;
	Ar << DetailMode;
}

void FFastGeoComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	check(!GetOwnerContainer()->IsRegistered());
	WorldTransform = LocalTransform * InTransform;
}

void FFastGeoComponent::OnAsyncCreatePhysicsState()
{
#ifdef FASTGEO_DEBUG_PHYSICS
	check(IsCollisionEnabled());
	check(PhysicsStateCreation == EPhysicsStateCreation::NotCreated);
	PhysicsStateCreation = EPhysicsStateCreation::Creating;
#endif
}

void FFastGeoComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
#ifdef FASTGEO_DEBUG_PHYSICS
	check(PhysicsStateCreation == EPhysicsStateCreation::Creating);
	PhysicsStateCreation = EPhysicsStateCreation::Created;
#endif
}

void FFastGeoComponent::OnAsyncDestroyPhysicsStateBegin_GameThread()
{
#ifdef FASTGEO_DEBUG_PHYSICS
	check(PhysicsStateCreation == EPhysicsStateCreation::Created);
	PhysicsStateCreation = EPhysicsStateCreation::Destroying;
#endif
}

void FFastGeoComponent::OnAsyncDestroyPhysicsState()
{
#ifdef FASTGEO_DEBUG_PHYSICS
	check(IsCollisionEnabled());
#endif
}

void FFastGeoComponent::OnAsyncDestroyPhysicsStateEnd_GameThread()
{
#ifdef FASTGEO_DEBUG_PHYSICS
	check(PhysicsStateCreation == EPhysicsStateCreation::Destroying);
	PhysicsStateCreation = EPhysicsStateCreation::NotCreated;
#endif
}

bool FFastGeoComponent::IsRenderStateCreated() const
{
	return ProxyState == EProxyCreationState::Created;
}

bool FFastGeoComponent::IsRenderStateDelayed() const
{
	return ProxyState == EProxyCreationState::Delayed;
}

bool FFastGeoComponent::IsRenderStateDirty() const
{
	return bRenderStateDirty;
}

void FFastGeoComponent::MarkRenderStateDirty(bool bEvenIfNotCreated)
{
	check(IsInGameThread());
	if ((IsRenderStateCreated() || IsRenderStateDelayed() || bEvenIfNotCreated) && !IsRenderStateDirty())
	{
		bRenderStateDirty = true;

		UFastGeoWorldSubsystem* WorldSubsystem = GetWorld()->GetSubsystemChecked<UFastGeoWorldSubsystem>();
		WorldSubsystem->AddToComponentsPendingRecreate(this);
	}
}

bool FFastGeoComponent::ShouldComponentAddToRenderScene() const
{
	if (!FApp::CanEverRender())
	{
		return false;
	}

	// If the detail mode setting allows it, add it to the scene.
	return DetailMode <= GetCachedScalabilityCVars().DetailMode;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
bool FFastGeoComponent::ShouldPrecachePSOs() const
{
	return FApp::CanEverRender() && IsComponentPSOPrecachingEnabled() && !GetPSOPrecacheComponentData().IsPSOPrecacheCalled();
}
#endif