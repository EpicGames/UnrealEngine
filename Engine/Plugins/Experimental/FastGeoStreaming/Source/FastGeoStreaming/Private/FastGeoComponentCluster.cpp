// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoComponentCluster.h"
#include "FastGeoContainer.h"
#include "SceneInterface.h"
#include "PrimitiveSceneInfo.h"
#include "FastGeoWeakElement.h"
#include "Containers/Ticker.h"

const FFastGeoElementType FFastGeoComponentCluster::Type(&IFastGeoElement::Type);

FFastGeoComponentCluster::FFastGeoComponentCluster(UFastGeoContainer* InOwner, FName InName, FFastGeoElementType InType)
	: Super(InType)
	, Owner(InOwner)
	, Name(InName.ToString())
	, ComponentClusterIndex(INDEX_NONE)
{
}

FFastGeoComponentCluster::FFastGeoComponentCluster(const FFastGeoComponentCluster& InOther)
	: Super(InOther.ElementType)
{
	ComponentClusterIndex = InOther.ComponentClusterIndex;
	Owner = InOther.Owner;
	Name = InOther.Name;
	StaticMeshComponents = InOther.StaticMeshComponents;
	InstancedStaticMeshComponents = InOther.InstancedStaticMeshComponents;
	SkinnedMeshComponents = InOther.SkinnedMeshComponents;
	InstancedSkinnedMeshComponents = InOther.InstancedSkinnedMeshComponents;
	ProceduralISMComponents = InOther.ProceduralISMComponents;
	DecalComponents = InOther.DecalComponents;
	RectLightComponents = InOther.RectLightComponents;
	SpotLightComponents = InOther.SpotLightComponents;
	PointLightComponents = InOther.PointLightComponents;
}

FArchive& operator<<(FArchive& Ar, FFastGeoComponentCluster& ComponentCluster)
{
	ComponentCluster.Serialize(Ar);
	return Ar;
}

void FFastGeoComponentCluster::Serialize(FArchive& Ar)
{
	Ar << Name;
	Ar << ComponentClusterIndex;
	ForEachComponentArray([&Ar](auto& Array) { Ar << Array; });
}

void FFastGeoComponentCluster::InitializeDynamicProperties()
{
	ForEachComponent([this](FFastGeoComponent& Component)
	{
		Component.SetOwnerComponentCluster(this);
		Component.InitializeDynamicProperties();
	});
}

void FFastGeoComponentCluster::SetOwnerContainer(UFastGeoContainer* InOwner)
{
	check(InOwner);
	Owner = InOwner;
}

void FFastGeoComponentCluster::SetComponentClusterIndex(int32 InComponentClusterIndex)
{
	ComponentClusterIndex = InComponentClusterIndex;
}

UFastGeoContainer* FFastGeoComponentCluster::GetOwnerContainer() const
{
	return Owner;
}

FFastGeoComponent* FFastGeoComponentCluster::GetComponent(uint32 InComponentTypeID, int32 InComponentIndex)
{
	if (FFastGeoInstancedStaticMeshComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return InstancedStaticMeshComponents.IsValidIndex(InComponentIndex) ? &InstancedStaticMeshComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoStaticMeshComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return StaticMeshComponents.IsValidIndex(InComponentIndex) ? &StaticMeshComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoInstancedSkinnedMeshComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return InstancedSkinnedMeshComponents.IsValidIndex(InComponentIndex) ? &InstancedSkinnedMeshComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoSkinnedMeshComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return SkinnedMeshComponents.IsValidIndex(InComponentIndex) ? &SkinnedMeshComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoProceduralISMComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return ProceduralISMComponents.IsValidIndex(InComponentIndex) ? &ProceduralISMComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoDecalComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return DecalComponents.IsValidIndex(InComponentIndex) ? &DecalComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoRectLightComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return RectLightComponents.IsValidIndex(InComponentIndex) ? &RectLightComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoSpotLightComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return SpotLightComponents.IsValidIndex(InComponentIndex) ? &SpotLightComponents[InComponentIndex] : nullptr;
	}
	else if (FFastGeoPointLightComponent::Type.IsSameTypeID(InComponentTypeID))
	{
		return PointLightComponents.IsValidIndex(InComponentIndex) ? &PointLightComponents[InComponentIndex] : nullptr;
	}
	check(false);
	return nullptr;
}

ULevel* FFastGeoComponentCluster::GetLevel() const
{
	return GetOwnerContainer()->GetLevel();
}

void FFastGeoComponentCluster::UpdateVisibility()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponentCluster::UpdateVisibility);

	TArray<FFastGeoPrimitiveComponent*> ShowComponents;
	TArray<FFastGeoPrimitiveComponent*> HideComponents;
	ForEachComponent<FFastGeoComponent>([&ShowComponents, &HideComponents](FFastGeoComponent& Component)
	{
		if (FFastGeoPrimitiveComponent* PrimitiveComponent = Component.CastTo<FFastGeoPrimitiveComponent>())
		{
			FWriteScopeLock WriteLock(*PrimitiveComponent->Lock.Get());
			const bool bOldIsDrawnInGame = PrimitiveComponent->IsDrawnInGame();
			PrimitiveComponent->UpdateVisibility();
			const bool bIsDrawnInGame = PrimitiveComponent->IsDrawnInGame();
			if (bIsDrawnInGame != bOldIsDrawnInGame)
			{
				if (FPrimitiveSceneProxy* SceneProxy = PrimitiveComponent->GetSceneProxy())
				{
					if (bIsDrawnInGame)
					{
						ShowComponents.Add(PrimitiveComponent);
					}
					else
					{
						HideComponents.Add(PrimitiveComponent);
					}
				}
			}
		}
		else
		{
			Component.UpdateVisibility();
		}
	});

	UpdateVisibility_Internal(MoveTemp(ShowComponents), MoveTemp(HideComponents));
}

void FFastGeoComponentCluster::ForceUpdateVisibility(const TArray<FFastGeoPrimitiveComponent*>& Components, int32 UpdateCounter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponentCluster::ForceUpdateVisibility);

	TArray<FFastGeoPrimitiveComponent*> ShowComponents;
	TArray<FFastGeoPrimitiveComponent*> HideComponents;
	for (FFastGeoPrimitiveComponent* Component : Components)
	{
		FWriteScopeLock WriteLock(*Component->Lock.Get());
		Component->UpdateVisibility();
		const bool bIsDrawnInGame = Component->IsDrawnInGame();
		if (FPrimitiveSceneProxy* SceneProxy = Component->GetSceneProxy())
		{
			if (bIsDrawnInGame)
			{
				ShowComponents.Add(Component);
			}
			else
			{
				HideComponents.Add(Component);
			}
		}
	}

	UpdateVisibility_Internal(MoveTemp(ShowComponents), MoveTemp(HideComponents), ++UpdateCounter);
}

void FFastGeoComponentCluster::ForceUpdateVisibilityOrDeferTask(const TArray<FFastGeoPrimitiveComponent*>& Components, int32 UpdateCounter)
{
	if (UpdateCounter <= 1)
	{
		ForceUpdateVisibility(Components, UpdateCounter);
		return;
	}

	// If primitives are not yet registered in the scene, immediate retries during blocking loads can loop indefinitely.
	// Defer to the next frame to allow pending AddPrimitive render commands to complete.
	FWeakFastGeoComponentCluster ClusterWeak(this);
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([ClusterWeak, Components, UpdateCounter](float) -> bool
	{
		if (FFastGeoComponentCluster* Cluster = ClusterWeak.Get())
		{
			Cluster->ForceUpdateVisibility(Components, UpdateCounter);
		}
		return false;
	}));
}

void FFastGeoComponentCluster::UpdateVisibility_Internal(TArray<FFastGeoPrimitiveComponent*>&& ShowComponents, TArray<FFastGeoPrimitiveComponent*>&& HideComponents, int32 UpdateCounter)
{
	if (!ShowComponents.IsEmpty() || !HideComponents.IsEmpty())
	{
		FWeakFastGeoComponentCluster ClusterWeak(this);
		ENQUEUE_RENDER_COMMAND()([ClusterWeak, UpdateCounter, ShowComponents = MoveTemp(ShowComponents), HideComponents = MoveTemp(HideComponents)](FRHICommandListBase&) mutable
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponentCluster::UpdateVisibility_RenderThread);

			if (!ClusterWeak.Get())
			{
				return;
			}

			TArray<FFastGeoPrimitiveComponent*> NotReadyComponents;
			auto ProcessComponents = [&NotReadyComponents](TArray<FFastGeoPrimitiveComponent*>& InOutComponents, bool bShow)
			{
				for (FFastGeoPrimitiveComponent* Component : InOutComponents)
				{
					FReadScopeLock ReadLock(*Component->Lock.Get());
					if (FPrimitiveSceneProxy* Proxy = Component->GetSceneProxy())
					{
						// Test whether the primitive was added to the scene (or is pending)
						FPrimitiveSceneInfo* PrimitiveSceneInfo = Proxy->GetPrimitiveSceneInfo();
						if (!PrimitiveSceneInfo->IsIndexValid())
						{
							NotReadyComponents.Add(Component);
						}
						else
						{
							Proxy->GetScene().UpdatePrimitivesIsDrawn_RenderThread(MakeArrayView(&Proxy, 1), bShow);
						}
					}
				}
			};

			ProcessComponents(ShowComponents, true);
			ProcessComponents(HideComponents, false);

			if (!NotReadyComponents.IsEmpty())
			{
				UE::Tasks::Launch(TEXT("ForceUpdateVisibility"), [ClusterWeak, UpdateCounter, NotReadyComponents = MoveTemp(NotReadyComponents)]()
				{
					if (FFastGeoComponentCluster* Cluster = ClusterWeak.Get())
					{
						Cluster->ForceUpdateVisibilityOrDeferTask(NotReadyComponents, UpdateCounter);
					}
				}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
			}
		});
	}
}

FFastGeoComponent& FFastGeoComponentCluster::AddComponent(FFastGeoElementType InComponentType)
{
	FFastGeoComponent* NewComponent = nullptr;

	if (InComponentType == FFastGeoInstancedStaticMeshComponent::Type)
	{
		NewComponent = &InstancedStaticMeshComponents.Emplace_GetRef(InstancedStaticMeshComponents.Num());
	}
	else if (InComponentType == FFastGeoStaticMeshComponent::Type)
	{
		NewComponent = &StaticMeshComponents.Emplace_GetRef(StaticMeshComponents.Num());
	}
	else if (InComponentType == FFastGeoSkinnedMeshComponent::Type)
	{
		NewComponent = &SkinnedMeshComponents.Emplace_GetRef(SkinnedMeshComponents.Num());
	}
	else if (InComponentType == FFastGeoInstancedSkinnedMeshComponent::Type)
	{
		NewComponent = &InstancedSkinnedMeshComponents.Emplace_GetRef(InstancedSkinnedMeshComponents.Num());
	}
	else if (InComponentType == FFastGeoProceduralISMComponent::Type)
	{
		NewComponent = &ProceduralISMComponents.Emplace_GetRef(ProceduralISMComponents.Num());
	}
	else if (InComponentType == FFastGeoDecalComponent::Type)
	{
		NewComponent = &DecalComponents.Emplace_GetRef(DecalComponents.Num());
	}
	else if (InComponentType == FFastGeoRectLightComponent::Type)
	{
		NewComponent = &RectLightComponents.Emplace_GetRef(RectLightComponents.Num());
	}
	else if (InComponentType == FFastGeoSpotLightComponent::Type)
	{
		NewComponent = &SpotLightComponents.Emplace_GetRef(SpotLightComponents.Num());
	}
	else if (InComponentType == FFastGeoPointLightComponent::Type)
	{
		NewComponent = &PointLightComponents.Emplace_GetRef(PointLightComponents.Num());
	}
	else
	{
		unimplemented(); // Should never be reached, missing type handling
	}

	if (NewComponent)
	{
		NewComponent->SetOwnerComponentCluster(this);
	}

	return *NewComponent;
}

bool FFastGeoComponentCluster::HasComponents() const
{
	bool bHasAny = false;
	ForEachComponentArray([&bHasAny](const auto& Array) -> bool
	{
		bHasAny = Array.Num() > 0;
		return !bHasAny;
	});
	return bHasAny;
}

#if WITH_EDITOR
void FFastGeoComponentCluster::SerializeWithComponentFilter(FArchive& Ar, TFunctionRef<bool(const FFastGeoComponent&)> ShouldSerializeComponent)
{
	Ar << Name;
	Ar << ComponentClusterIndex;
	ForEachComponentArray([&](auto& Array)
	{
		for (auto& Component : Array)
		{
			if (ShouldSerializeComponent(Component))
			{
				Ar << Component;
			}
		}
	});
}
#endif

#if !WITH_EDITOR
int32 FFastGeoComponentCluster::StripRenderOnlyComponents()
{
	check(!FApp::CanEverRender());

	int32 NumStripped = 0;

	// Decals and lights are purely visual -- empty unconditionally
	auto StripAllComponents = [&NumStripped](auto& Array)
	{
		NumStripped += Array.Num();
		Array.Empty();
	};
	StripAllComponents(DecalComponents);
	StripAllComponents(PointLightComponents);
	StripAllComponents(SpotLightComponents);
	StripAllComponents(RectLightComponents);

	// Strip remaining components without collision or navigation relevance.
	// Already-emptied arrays are skipped by RemoveAll (no-op on empty arrays).
	ForEachComponentArray([&NumStripped](auto& Array)
	{
		int32 NumBefore = Array.Num();
		Array.RemoveAll([](const auto& Component) -> bool
		{
			const FFastGeoPrimitiveComponent* Primitive = Component.template CastTo<FFastGeoPrimitiveComponent>();
			return !Primitive || (!Primitive->IsCollisionEnabled() && !Primitive->IsNavigationRelevant());
		});
		NumStripped += NumBefore - Array.Num();
	});

	return NumStripped;
}
#endif