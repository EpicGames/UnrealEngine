// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoContainer.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoHLOD.h"
#include "FastGeoComponent.h"
#include "FastGeoStreamingModule.h"
#include "FastGeoSkinnedMeshComponent.h"
#include "FastGeoSurrogateActor.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoDestroyRenderStateContext.h"
#include "FastGeoLog.h"
#include "AI/Navigation/NavigationElement.h"
#include "Animation/TransformProviderData.h"
#include "Async/ParallelFor.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/TextureLightProfile.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Timeout.h"
#include "NavigationSystem.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "PSOPrecache.h"
#include "PSOPrecacheSettings.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "SceneInterface.h"
#include "Templates/Invoke.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartition.h"

namespace FastGeo::Private
{
	static int32 GAsyncRenderStateTaskParallelWorkerCount = 1;
	static FAutoConsoleVariableRef CVarAsyncRenderStateTaskParallelWorkerCount(
		TEXT("FastGeo.AsyncRenderStateTask.ParallelWorkerCount"),
		GAsyncRenderStateTaskParallelWorkerCount,
		TEXT("Set the max number of workers to use when creating FastGeo render state. ")
		TEXT("Only taken into account if value is greater than 1."),
		ECVF_Default);

#if !WITH_EDITOR
	static bool GStripRenderOnlyComponentsOnNonRenderingProcess = true;
	static FAutoConsoleVariableRef CVarStripRenderOnlyComponentsOnNonRenderingProcess(
		TEXT("FastGeo.StripRenderOnlyComponentsOnNonRenderingProcess"),
		GStripRenderOnlyComponentsOnNonRenderingProcess,
		TEXT("On non-rendering processes (e.g. dedicated server), strip purely render-only components ")
		TEXT("(no collision, no navigation) from FastGeo containers to reduce memory usage."),
		ECVF_ReadOnly);
#endif


	class FAssetRemapArchive : public FArchiveProxy
	{
	public:
		FAssetRemapArchive(FArchive& InArchive, TArray<TObjectPtr<UObject>>& InUniqueAssetsArray)
			: FArchiveProxy(InArchive)
			, UniqueAssetsArray(InUniqueAssetsArray)
		{
			// For some unknown reason, copy constructor resets ArIsFilterEditorOnly flag copied from the input archive (see FArchiveState(const FArchiveState&))
			ArIsFilterEditorOnly = InArchive.ArIsFilterEditorOnly;
			for (int32 Index = 0, Num = UniqueAssetsArray.Num(); Index < Num; ++Index)
			{
				UniqueAssets.Add(UniqueAssetsArray[Index], Index);
			}
		}

		virtual FArchive& operator<<(UObject*& Obj) override
		{
			if (IsLoading())
			{
				int32 Index;
				*this << Index;
				Obj = UniqueAssetsArray.IsValidIndex(Index) ? UniqueAssetsArray[Index] : nullptr;
			}
			else if (IsSaving())
			{
				int32 Index = INDEX_NONE;
				if (int32* ExistingIndex = Obj ? UniqueAssets.Find(Obj) : nullptr)
				{
					Index = *ExistingIndex;
				}
				*this << Index;
			}
			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Obj) override
		{
			UObject* ObjPtr = Obj.Get();
			FArchive& Result = operator<<(ObjPtr);
			Obj = ObjPtr;
			return Result;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override { unimplemented(); return *this; }
		virtual FArchive& operator<<(FSoftObjectPtr& AssetPtr) override { unimplemented(); return *this; }
		virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override { unimplemented(); return *this; }
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override { unimplemented(); return *this; }

	private:
		TMap<UObject*, int32> UniqueAssets;
		TArray<TObjectPtr<UObject>>& UniqueAssetsArray;
	};

	class FAsyncTaskAssetReferenceManager : public FGCObject
	{
	public:
		TMap<FGuid, TArray<TObjectPtr<UObject>>> TasksAssets;

		// Made an on-demand singleton rather than a static global, to avoid issues with FGCObject initialization
		static FAsyncTaskAssetReferenceManager& Get()
		{
			static FAsyncTaskAssetReferenceManager Manager;
			return Manager;
		}

		void RegisterTaskAssets(const FGuid& TaskId, const TArray<TObjectPtr<UObject>>& Assets)
		{
			TasksAssets.Add(TaskId, Assets);
		}

		void UnregisterTask(const FGuid& TaskId)
		{
			TasksAssets.Remove(TaskId);
		}

		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			for (auto& Pair : TasksAssets)
			{
				Collector.AddReferencedObjects(Pair.Value);
			}
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FAsyncTaskAssetReferenceManager");
		}
	};

	struct FAsyncTaskWithAssetRefs : public TSharedFromThis<FAsyncTaskWithAssetRefs>
	{
		explicit FAsyncTaskWithAssetRefs(const TArray<TObjectPtr<UObject>>& InAssets)
			: TaskId(FGuid::NewGuid())
			, Assets(InAssets)
		{
			FAsyncTaskAssetReferenceManager::Get().RegisterTaskAssets(TaskId, Assets);
		}

		~FAsyncTaskWithAssetRefs()
		{
			// Ensure unregistration happens on the game thread
			UE::Tasks::Launch(TEXT("UnregisterFastGeoTask"), [MyTaskId = TaskId]()
			{
				FAsyncTaskAssetReferenceManager::Get().UnregisterTask(MyTaskId);
			}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
		}

		template<typename TaskBodyType>
		static UE::Tasks::TTask<TInvokeResult_T<TaskBodyType>> Launch(const TCHAR* TaskName, const TArray<TObjectPtr<UObject>>& AssetRefs, TaskBodyType&& Work, LowLevelTasks::ETaskPriority Priority)
		{
			TSharedRef<FAsyncTaskWithAssetRefs> TaskData = MakeShared<FAsyncTaskWithAssetRefs>(AssetRefs);

			return UE::Tasks::Launch(TaskName, [TaskData, Work = MoveTemp(Work)]() mutable
			{
				Work();
			}, Priority);
		}

	private:
		FGuid TaskId;
		TArray<TObjectPtr<UObject>> Assets;
	};

	// Sync mode calls End_GameThread once after all budget-sliced batches complete.
	// Reset the scan range to [0, Num) so End_GameThread processes all components,
	// not just the last batch (which would miss components delayed in earlier batches).
	static void ResetBatchForSyncCompletion(FFastGeoRenderStateBatch& State)
	{
		State.TotalNumProcessed = 0;
		State.NumProcessed = State.ComponentsToProcess.Num();
	}
}

void UFastGeoContainer::ApplyRegistrationTargetState()
{
	check(IsInGameThread());

	switch (RegistrationState)
	{
	case ERegistrationState::Unregistered:
		if (RegistrationTargetState == ERegistrationTargetState::Registered)
		{
			if (!bIsBeingAsyncDestroyed)
			{
				StartRegisterTransition();
			}
			UE_CLOGF(bIsBeingAsyncDestroyed, LogFastGeoStreaming, Warning, "Can't start registering FastGeoContainer because it's being destroyed.");
		}
		break;

	case ERegistrationState::Registered:
		if (RegistrationTargetState == ERegistrationTargetState::Unregistered)
		{
			StartUnregisterTransition();
		}
		break;

	default:
		break;
	}
}

void UFastGeoContainer::Register()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::Register);
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return;
	}
#endif

	if (!FFastGeoStreamingModule::IsFastGeoEnabled())
	{
		UE_LOGF(LogFastGeoStreaming, Warning, "UFastGeoContainer::Register() ignored because FastGeo is disabled.");
		return;
	}

	if (bIsBeingAsyncDestroyed)
	{
		UE_LOGF(LogFastGeoStreaming, Warning, "UFastGeoContainer::Register() ignored because FastGeoContainer is being destroyed.");
		return;
	}

	RegistrationTargetState = ERegistrationTargetState::Registered;
	ApplyRegistrationTargetState();

	// Recreate (UseFallbackMaterialUntilPSOPrecached) is post-registration and shouldn't prevent re-registration setup.
	if (HasAnyPendingRegistrationTasks())
	{
		Tick();
	}
}

void UFastGeoContainer::StartRegisterTransition()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::StartRegisterTransition);

	check(IsInGameThread());
#if WITH_EDITOR
	check(!IsRunningCookCommandlet());
#endif
	check(FFastGeoStreamingModule::IsFastGeoEnabled());
	check(!bIsBeingAsyncDestroyed);
	check(RegistrationTargetState == ERegistrationTargetState::Registered);
	check(RegistrationState == ERegistrationState::Unregistered);
	// Surrogate flag may be stale during FGlobalComponentReregisterContext (no actor-level callbacks).
	check(!IsUsingSurrogateComponents() || !SurrogateActor->HasRegisteredWithFastGeo() || GetWorldSubsystem()->IsReregistering());

	// PSO precaching is allowed to run before registration. Recreate from a prior registration cycle may still be in flight -- that's OK.
	constexpr bool bIncludePSOPrecaching = false;
	if (const bool bHasAnyPendingTasks = HasAnyPendingRegistrationTasks(bIncludePSOPrecaching))
	{
		checkf(!bHasAnyPendingTasks, TEXT("UFastGeoContainer::StartRegisterTransition() was called while the container still has pending tasks. This will lead to invalid behavior."));
		return;
	}

	RegistrationState = ERegistrationState::Registering;

	++RegistrationEpoch;
	PendingCreate.Reset();

#if WITH_EDITOR
	// In PIE we need to initialize dynamic properties as there's no serialization
	InitializeDynamicProperties(true);
#endif
	UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(this);
	const bool bApplyWorldTransform = WorldPartition && WorldPartition->HasInstanceTransform();
	const FTransform& Transform = WorldPartition ? WorldPartition->GetInstanceTransform() : FTransform::Identity;

	// Register clusters (FastGeoHLODs)
	// Apply world transform
	// Prepare PendingCreate.RenderState
	ForEachComponentCluster([bApplyWorldTransform, &Transform, this](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.OnRegister();

		ComponentCluster.ForEachComponent<FFastGeoComponent>([bApplyWorldTransform, &Transform, this](FFastGeoComponent& Component)
		{
			if (bApplyWorldTransform)
			{
				Component.ApplyWorldTransform(Transform);
			}

			if (Component.ShouldComponentAddToRenderScene())
			{
				PendingCreate.RenderState.ComponentsToProcess.Add(&Component);
			}
		});
	});

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Prepare PendingCreate.PSOPrecachingState
	PendingCreate.PrecachePSOState.Reset();
	if (!FFastGeoStreamingModule::IsAsyncRenderWorkAllowed() && !bPSOPrecachingInitiated)
	{
		if (IsComponentPSOPrecachingEnabled())
		{
			PendingCreate.PrecachePSOState.ComponentsToProcess = PendingCreate.RenderState.ComponentsToProcess;
			PendingCreate.PrecachePSOState.NumToProcess = PendingCreate.PrecachePSOState.ComponentsToProcess.Num();
		}
		bPSOPrecachingInitiated = (PendingCreate.PrecachePSOState.ComponentsToProcess.Num() == 0);
	}
#endif

	InitializePrerequisites();
	StartCreateRenderStateWork();
	StartCreatePhysicsStateWork();

	// If no work needed for any system, registration is already complete
	TryCompleteRegistration();

	const UNavigationSystemBase* NavigationSystem = GetWorld()->GetNavigationSystem();
	if (NavigationSystem && NavigationSystem->IsWorldInitDone())
	{
		RegisterToNavigationSystem();
	}
	else
	{
		UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().AddUObject(this, &UFastGeoContainer::OnNavigationInitDone);
	}
}

bool UFastGeoContainer::IsRegistering() const
{
	check(IsInGameThread());
	return RegistrationState == ERegistrationState::Registering;
}

bool UFastGeoContainer::IsRegistered() const
{
	check(IsInGameThread());
	return RegistrationState == ERegistrationState::Registered;
}

bool UFastGeoContainer::IsFullyRegistered() const
{
	check(IsInGameThread());
	// Recreate is post-registration background work -- container IS fully registered.
	return (RegistrationState == ERegistrationState::Registered) && !HasAnyPendingRegistrationTasks();
}

bool UFastGeoContainer::IsUnregistering() const
{
	check(IsInGameThread());
	return RegistrationState == ERegistrationState::Unregistering;
}

bool UFastGeoContainer::IsUnregistered() const
{
	check(IsInGameThread());
	return RegistrationState == ERegistrationState::Unregistered;
}

bool UFastGeoContainer::IsFullyUnregistered() const
{
	check(IsInGameThread());
	return (RegistrationState == ERegistrationState::Unregistered) && !HasAnyPendingTasks();
}

void UFastGeoContainer::OnRegisterCompleted()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnRegisterCompleted);
	// Verify that registration is truly complete
	check(!HasAnyPendingCreateTasks());
	check(RegistrationState == ERegistrationState::Registering);
	RegistrationState = ERegistrationState::Registered;

	// Only broadcast if we're not about to immediately unregister (e.g., DestroyRuntime called during registration).
	if (RegistrationTargetState == ERegistrationTargetState::Registered)
	{
		OnRegistered.Broadcast();
	}

	ApplyRegistrationTargetState();
}

void UFastGeoContainer::OnUnregisterCompleted()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnUnregisterCompleted);
	// Verify that unregistration is truly complete
	check(!HasAnyPendingDestroyTasks());
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	check(!HasAnyPendingRecreateTasks());
#endif
	check(RegistrationState == ERegistrationState::Unregistering);
	RegistrationState = ERegistrationState::Unregistered;

	AsyncContext.Reset();

	OnUnregistered.Broadcast();

	ApplyRegistrationTargetState();
}

void UFastGeoContainer::TryCompleteRegistration()
{
	if (RegistrationState == ERegistrationState::Registering && !HasAnyPendingCreateTasks())
	{
		OnRegisterCompleted();
	}
}

void UFastGeoContainer::TryCompleteUnregistration()
{
	if (RegistrationState == ERegistrationState::Unregistering
		&& !HasAnyPendingDestroyTasks()
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		&& !HasAnyPendingRecreateTasks()
#endif
		)
	{
		OnUnregisterCompleted();
	}
}

FFastGeoCreateRuntimeResult UFastGeoContainer::CreateRuntime(UWorld* InWorld, FName InDebugName, TFunctionRef<void(FFastGeoComponentCluster&)> InInitCluster, bool bInCollectReferences)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::CreateRuntime);
	check(IsInGameThread());

	FFastGeoCreateRuntimeResult Result;

	if (!FFastGeoStreamingModule::IsFastGeoEnabled())
	{
		UE_LOGF(LogFastGeoStreaming, Warning, "UFastGeoContainer::CreateRuntime() ignored because FastGeo is disabled.");
		return Result;
	}

	if (!IsValid(InWorld))
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::CreateRuntime() called with an invalid world.");
		return Result;
	}

	ULevel* PersistentLevel = InWorld->PersistentLevel;
	check(PersistentLevel && PersistentLevel->IsPersistentLevel());

	const FName UniqueName = MakeUniqueObjectName(PersistentLevel, UFastGeoContainer::StaticClass(), InDebugName);
	UFastGeoContainer* Container = NewObject<UFastGeoContainer>(PersistentLevel, UniqueName, RF_Transient);

	// Create the cluster, let the caller populate it with components, then add to container.
	FFastGeoComponentCluster TempCluster(Container, *FString::Printf(TEXT("FastGeoComponentCluster_%s"), *InDebugName.ToString()));
	InInitCluster(TempCluster);
	Container->AddComponentCluster(&TempCluster);

	Container->OnCreated(bInCollectReferences);
	Container->PrecachePSOs();
	Container->Register();
	Container->GetWorldSubsystem()->PushRuntimeContainerForRegistration(Container);

	// Populate result.
	Result.Container = Container;
	Result.Container->ForEachComponentCluster([&Result](FFastGeoComponentCluster& InComponentCluster)
	{
		InComponentCluster.ForEachComponent([&Result](FFastGeoComponent& InComponent)
		{
			Result.Components.Add(&InComponent);
		});
	});

	return Result;
}

void UFastGeoContainer::DestroyRuntime(UFastGeoContainer* InFastGeo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::DestroyRuntime);
	check(IsInGameThread());

	if (!IsValid(InFastGeo))
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::DestroyRuntime() called on an invalid container.");
		return;
	}

	if (!InFastGeo->GetWorld())
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::DestroyRuntime() called on a container with an invalid world.");
		return;
	}

	if (!InFastGeo->IsRuntime())
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::DestroyRuntime() is meant to be called on a runtime container.");
		return;
	}

	if (!InFastGeo->GetLevel()->IsPersistentLevel())
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::DestroyRuntime() expects the runtime container to be outered to the persistent level.");
		return;
	}

	if (InFastGeo->bIsBeingAsyncDestroyed)
	{
		UE_LOGF(LogFastGeoStreaming, Error, "UFastGeoContainer::DestroyRuntime() called on a container that is already being destroyed.");
		return;
	}

	InFastGeo->bIsBeingAsyncDestroyed = true;
	InFastGeo->Unregister();
	InFastGeo->GetWorldSubsystem()->PushRuntimeContainerForDestruction(InFastGeo);
}

void UFastGeoContainer::Unregister()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::Unregister);

#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return;
	}
#endif

	if (!FFastGeoStreamingModule::IsFastGeoEnabled())
	{
		UE_LOGF(LogFastGeoStreaming, Warning, "UFastGeoContainer::Unregister() ignored because FastGeo is disabled.");
		return;
	}

	RegistrationTargetState = ERegistrationTargetState::Unregistered;
	ApplyRegistrationTargetState();

	// Tick is invoked with bShouldWaitForCompletion=false so Unregister returns
	// without blocking GT. Any in-flight async work is expected to be driven to
	// completion by the world subsystem's per-frame Tick. Callers outside the
	// subsystem-driven flow must explicitly drive Tick() until HasAnyPendingTasks()
	// returns false (or call Tick(/*bWaitForCompletion=*/true)) to avoid stalling
	// the unregister transition.
	if (HasAnyPendingTasks())
	{
		Tick();
	}
}

void UFastGeoContainer::StartUnregisterTransition()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::StartUnregisterTransition);

	check(IsInGameThread());
#if WITH_EDITOR
	check(!IsRunningCookCommandlet());
#endif
	check(FFastGeoStreamingModule::IsFastGeoEnabled());
	check(RegistrationTargetState == ERegistrationTargetState::Unregistered);
	check(RegistrationState == ERegistrationState::Registered);
	check(!IsUsingSurrogateComponents() || SurrogateActor->HasRegisteredWithFastGeo());

	if (const bool bHasAnyPendingDestroyTasks = HasAnyPendingDestroyTasks())
	{
		checkf(!bHasAnyPendingDestroyTasks, TEXT("UFastGeoContainer::StartUnregisterTransition() was called while the container still has pending destroy tasks. This will lead to invalid behavior."));
		return;
	}

	RegistrationState = ERegistrationState::Unregistering;

	PendingDestroy.Reset();

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Clean up any PSO deferred/recreate state from the registration that just completed.
	// Without this, stale state would cause HasAnyPendingCreateTasks/HasAnyPendingRecreateTasks
	// to return true on re-registration (e.g., FGlobalComponentReregisterContext), triggering asserts.

	// Deferred create is part of registration and should be complete by now.
	check(PSODeferredPendingComponents.IsEmpty());
	check(PSODeferredReadyComponents.IsEmpty());
	check(PSODeferredCreateState.ComponentsToProcess.IsEmpty());
	PSODeferredPendingComponents.Reset();
	PSODeferredReadyComponents.Reset();
	PSODeferredCreateState.Reset();
	GetWorldSubsystem()->UnregisterContainerWithPSODeferredWork(this);

	// If a recreate job is in flight, don't touch PSORecreateState (worker may be reading it).
	// OnRecreateRenderStateEnd_GameThread will detect RegistrationState == Unregistering and abort,
	// and will call UnregisterContainerWithAsyncPSORecreateWork at that point.
	// bIsProcessingPSORecreate stays true so HasAnyPendingRecreateTasks() returns true until
	// the in-flight job completes and cleans up.
	if (!bIsProcessingPSORecreate)
	{
		check(PSORecreateState.ComponentsToProcess.IsEmpty());
		PSORecreateState.Reset();
		GetWorldSubsystem()->UnregisterContainerWithAsyncPSORecreateWork(this);
	}
	PSORecreateComponents.Reset();
#endif

	ForEachComponentCluster([this](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.OnUnregister();

		ComponentCluster.ForEachComponent<FFastGeoComponent>([this](FFastGeoComponent& Component)
		{
			PendingDestroy.RenderState.ComponentsToProcess.Add(&Component);
		});
	});

	UnregisterFromNavigationSystem();
		
	StartDestroyRenderStateWork();
	StartDestroyPhysicsStateWork();

	// If no work needed for any system, unregistration is already complete
	TryCompleteUnregistration();
}

bool UFastGeoContainer::TickPrecachePSO_Sync(const UE::FTimeout& Timeout)
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (!bPSOPrecachingInitiated)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::TickPrecachePSO_Sync);
		check(!FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());

		bPSOPrecachingInitiated = FastGeo::AdvanceRenderStateBudgeted(PendingCreate.PrecachePSOState, Timeout, [](FFastGeoComponent* Component)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponent_PrecachePSOs);
			Component->PrecachePSOs_Concurrent();
		});

		if (bPSOPrecachingInitiated)
		{
			PendingCreate.PrecachePSOState.Reset();
		}
	}
	return bPSOPrecachingInitiated;
#else
	return true;
#endif
}

bool UFastGeoContainer::TickCreateRenderState_Sync(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::TickCreateRenderState_Sync);
	check(IsInGameThread());
	check(!FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());

	if (PendingCreate.RenderState.ComponentsToProcess.Num() > 0 && PendingCreate.RenderState.NumToProcess == 0)
	{
		OnCreateRenderStateBegin_GameThread();
	}

	const bool bCompleted = FastGeo::AdvanceRenderStateBudgeted(PendingCreate.RenderState, Timeout, [](FFastGeoComponent* Component)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponent_CreateRenderState);
		Component->CreateRenderState(nullptr);
	});

	if (bCompleted)
	{
		FastGeo::Private::ResetBatchForSyncCompletion(PendingCreate.RenderState);
		OnCreateRenderStateEnd_GameThread();
	}

	return bCompleted;
}

bool UFastGeoContainer::TickDestroyRenderState_Sync(const UE::FTimeout& Timeout)
{
	if (Timeout.IsExpired())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::TickDestroyRenderState_Sync);
	check(IsInGameThread());
	check(!FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());

	FFastGeoRenderStateBatch& State = PendingDestroy.RenderState;
	if (State.ComponentsToProcess.Num() > 0 && State.NumToProcess == 0)
	{
		OnDestroyRenderStateBegin_GameThread();
	}
	
	static constexpr int32 MaxComponentPerIteration = 16;
	const TArray<FFastGeoComponent*>& Components = State.ComponentsToProcess;
	const int32 ComponentsCount = Components.Num();
	while (!IsDestroyRenderStateCompleted())
	{
		if (Timeout.IsExpired())
		{
			return false;
		}

		{
			FFastGeoDestroyRenderStateContext Context(GetWorld()->Scene);
			const int32 StartIndex = State.TotalNumProcessed;
			const int32 EndIndex = FMath::Min<int32>(StartIndex + MaxComponentPerIteration, ComponentsCount);
			for (int Index = StartIndex; Index < EndIndex; ++Index)
			{
				Components[Index]->DestroyRenderState(&Context);
			}
			State.TotalNumProcessed += (EndIndex - StartIndex);
			State.NumProcessed = State.TotalNumProcessed;
			State.NumToProcess = State.ComponentsToProcess.Num() - State.TotalNumProcessed;
		}
	}

	check(IsDestroyRenderStateCompleted());
	// Force NumProcessed to zero because OnDestroyRenderStateEnd_GameThread was designed to increment TotalNumProcessed
	State.NumProcessed = 0;
	OnDestroyRenderStateEnd_GameThread();
	return true;
}

void UFastGeoContainer::StartCreatePhysicsStateWork()
{
	if (const bool bHasPhysicsWork = !CollisionComponents.IsEmpty())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->PushAsyncCreatePhysicsStateJobs(this);
	}
}

void UFastGeoContainer::StartDestroyPhysicsStateWork()
{
	if (const bool bHasPhysicsWork = !CollisionComponents.IsEmpty())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->PushAsyncDestroyPhysicsStateJobs(this);
	}
}

void UFastGeoContainer::TickPhysicsWork(bool bWaitForCompletion)
{
	if (FPhysScene* PhysScene = GetPhysicsScene())
	{
		PhysScene->ProcessAsyncPhysicsStateJobs(bWaitForCompletion);
	}
}

void UFastGeoContainer::StartPrecachePSOWork()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed() && !bPSOPrecachingInitiated)
	{
		bPSOPrecachingInitiated = true;
		if (IsComponentPSOPrecachingEnabled())
		{
			UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
			WorldSubsystem->PushAsyncPrecachePSOsJob(this);
		}
	}
#endif
}

void UFastGeoContainer::StartCreateRenderStateWork()
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed() && !IsInitialCreateRenderStatePassCompleted())
	{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		// PSO precaching should have been started by UFastGeoWorldSubsystem::OnLevelStreamingStateChanged
		if (!ensure(bPSOPrecachingInitiated))
		{
			StartPrecachePSOWork();
			check(bPSOPrecachingInitiated);
		}
#endif
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->PushAsyncCreateRenderStateJob(this);
	}
}

void UFastGeoContainer::StartDestroyRenderStateWork()
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed() && !IsDestroyRenderStateCompleted())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->PushAsyncDestroyRenderStateJob(this);
	}
}

bool UFastGeoContainer::IsPrecachePSOCompleted() const
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	return bPSOPrecachingInitiated && PendingCreate.PrecachePSOState.IsCompleted();
#else
	return true;
#endif
}

bool UFastGeoContainer::IsInitialCreateRenderStatePassCompleted() const
{
	return PendingCreate.RenderState.IsCompleted();
}

bool UFastGeoContainer::IsCreateRenderStateFullyCompleted() const
{
	if (!IsInitialCreateRenderStatePassCompleted())
	{
		return false;
	}
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (HasPSODeferredComponents())
	{
		return false;
	}
	if (bIsProcessingPSODeferredCreate)
	{
		return false;
	}
	check(PSODeferredCreateState.IsCompleted());
#endif
	return true;
}

bool UFastGeoContainer::IsDestroyRenderStateCompleted() const
{
	return PendingDestroy.RenderState.IsCompleted();
}

void UFastGeoContainer::TickRenderWork(bool bWaitForCompletion)
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->ProcessAsyncRenderStateJobs(bWaitForCompletion);
	}
	else
	{
		TickRenderWork_Sync(bWaitForCompletion);
	}
}

void UFastGeoContainer::TickRenderWork_Sync(bool bWaitForCompletion)
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::TickRenderWork_Sync);
	check(IsInGameThread());

	switch (RegistrationState)
	{
	case ERegistrationState::Registering:
		{
			if (!IsPrecachePSOCompleted() || !IsInitialCreateRenderStatePassCompleted())
			{
				const UE::FTimeout Timeout = bWaitForCompletion ? UE::FTimeout::Never() : GetWorld()->GetAddToWorldTimeout();
				if (TickPrecachePSO_Sync(Timeout))
				{
					check(IsPrecachePSOCompleted());
					if (TickCreateRenderState_Sync(Timeout))
					{
						check(IsInitialCreateRenderStatePassCompleted());
					}
				}
			}
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
			// Deferred create: components whose PSOs were not ready during initial create.
			// Uses AddToWorld timeout since deferred create is part of registration.
			if (IsInitialCreateRenderStatePassCompleted() && !PSODeferredCreateState.ComponentsToProcess.IsEmpty())
			{
				const UE::FTimeout Timeout = bWaitForCompletion ? UE::FTimeout::Never() : GetWorld()->GetAddToWorldTimeout();
				TickDeferredCreate_Sync(Timeout);
			}
#endif
		}
		break;
	case ERegistrationState::Unregistering:
		{
			if (!IsDestroyRenderStateCompleted())
			{
				const UE::FTimeout Timeout = bWaitForCompletion ? UE::FTimeout::Never() : GetWorld()->GetRemoveFromWorldTimeout();
				if (TickDestroyRenderState_Sync(Timeout))
				{
					check(IsDestroyRenderStateCompleted());
				}
			}
		}
		break;
	default:
		// Recreate is valid in the default state (post-registration) and handled below.
		check(!HasAnyPendingRegistrationTasks(/*bIncludePSOPrecaching=*/ false));
		break;
	}

	// Note: sync-mode strategy 2 recreate (UseFallbackMaterialUntilPSOPrecached) is handled by MarkRenderStateDirty -> ProcessPendingRecreate, not here.
	// PushPSORecreateComponent is async-only (asserted). See OnComponentPSOPrecacheCompleted for the routing.
}

void UFastGeoContainer::Tick(bool bWaitForCompletion)
{
	check(IsInGameThread());
	if (!ensureMsgf(GetWorld(), TEXT("UFastGeoContainer::Tick() skipped, world is not valid.")))
	{
		return;
	}

	// Callers may pass bWaitForCompletion=false even when GT is blocked (e.g., AddToWorld called from BlockTillLevelStreamingCompleted).
	// Detect the blocking state from the world to ensure we force-create PSO-deferred proxies instead of parking indefinitely.
	const bool bShouldWaitForCompletion = bWaitForCompletion || GetWorld()->GetIsInBlockTillLevelStreamingCompleted();

	if (!HasAnyPendingTasks())
	{
		ApplyRegistrationTargetState();
	}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// If we enter a blocking wait while already parked with PSO-deferred components, un-park and force-create.
	// GT is blocked (e.g., BlockTillLevelStreamingCompleted) and cannot pump the task graph, so PSO callbacks will never fire.
	if (bShouldWaitForCompletion && HasPSODeferredComponents())
	{
		// Wait for any in-flight deferred create job to complete before touching PSODeferredCreateState (a worker thread could be reading it).
		TickRenderWork(bShouldWaitForCompletion);
		ForceCreateDeferredComponents();
	}
#endif

	do
	{
		TickRenderWork(bShouldWaitForCompletion);
		TickPhysicsWork(bShouldWaitForCompletion);

		if (bShouldWaitForCompletion && !HasAnyPendingTasks())
		{
			ApplyRegistrationTargetState();
		}
	} while (bShouldWaitForCompletion && HasAnyPendingTasks());
}

bool UFastGeoContainer::HasAnyPendingTasks(bool bIncludePSOPrecaching) const
{
	return HasAnyPendingCreateTasks(bIncludePSOPrecaching) || HasAnyPendingDestroyTasks()
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		|| HasAnyPendingRecreateTasks()
#endif
		;
}

bool UFastGeoContainer::HasAnyPendingRegistrationTasks(bool bIncludePSOPrecaching) const
{
	return HasAnyPendingCreateTasks(bIncludePSOPrecaching) || HasAnyPendingDestroyTasks();
}

bool UFastGeoContainer::HasAnyPendingCreateTasks(bool bIncludePSOPrecaching) const
{
	// Physics state can have no PendingState but OnAsyncCreatePhysicsStateEnd_GameThread has not been called yet
	return (PhysicsState == EPhysicsState::Creating) || PendingCreate.HasAnyPendingState(bIncludePSOPrecaching) || !IsCreateRenderStateFullyCompleted();
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
bool UFastGeoContainer::HasAnyPendingRecreateTasks() const
{
	return bIsProcessingPSORecreate || !PSORecreateComponents.IsEmpty();
}
#endif

bool UFastGeoContainer::HasAnyPendingDestroyTasks() const
{
	// Physics state can have no PendingState but OnAsyncDestroyPhysicsStateEnd_GameThread has not been called yet
	return (PhysicsState == EPhysicsState::Destroying) || PendingDestroy.HasAnyPendingState();
}

FFastGeoComponentCluster* UFastGeoContainer::GetComponentCluster(uint32 InComponentClusterTypeID, int32 InComponentClusterIndex)
{
	if (FFastGeoHLOD::Type.IsSameTypeID(InComponentClusterTypeID))
	{
		return HLODs.IsValidIndex(InComponentClusterIndex) ? &HLODs[InComponentClusterIndex] : nullptr;
	}
	else if (FFastGeoComponentCluster::Type.IsSameTypeID(InComponentClusterTypeID))
	{
		return ComponentClusters.IsValidIndex(InComponentClusterIndex) ? &ComponentClusters[InComponentClusterIndex] : nullptr;
	}
	check(false);
	return nullptr;
}

TArray<IPrimitiveComponent*> UFastGeoContainer::GetPrimitiveComponents() const
{
	return PrimitiveComponents;
}

TArray<IStaticMeshComponent*> UFastGeoContainer::GetStaticMeshComponents() const
{
	return StaticMeshComponents;
}

ULevel* UFastGeoContainer::GetLevel() const
{
	return GetOuterULevel();
}

UWorld* UFastGeoContainer::GetWorld() const
{
	if (AsyncContext.IsInitialized())
	{
		return AsyncContext.GetWorld();
	}
	// Fallback walks the level outer chain, which is GT-mutable. Allowed only on GT, ParallelGT, or
	// the AsyncLoadingThread (Serialize/PostLoad can land here before the container is registered).
	// Worker threads dispatched via the FastGeo pipe MUST go through AsyncContext above; this branch
	// is unreachable for them.
	check(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelGameThread());
	ULevel* Level = GetLevel();
	return Level ? Level->GetWorld() : nullptr;
}

UFastGeoWorldSubsystem* UFastGeoContainer::GetWorldSubsystem() const
{
	if (AsyncContext.IsInitialized())
	{
		return AsyncContext.GetWorldSubsystem();
	}
	if (UWorld* World = GetWorld())
	{
		return World->GetSubsystemChecked<UFastGeoWorldSubsystem>();
	}
	return nullptr;
}

FSceneInterface* UFastGeoContainer::GetScene() const
{
	if (AsyncContext.IsInitialized())
	{
		return AsyncContext.GetScene();
	}
	if (UWorld* World = GetWorld())
	{
		return World->Scene;
	}
	return nullptr;
}

FPhysScene* UFastGeoContainer::GetPhysicsScene() const
{
	if (AsyncContext.IsInitialized())
	{
		return AsyncContext.GetPhysicsScene();
	}
	if (UWorld* World = GetWorld())
	{
		return World->GetPhysicsScene();
	}
	return nullptr;
}

float UFastGeoContainer::GetWorldTimeSeconds() const
{
	if (AsyncContext.IsInitialized())
	{
		return AsyncContext.GetTimeSeconds();
	}
	// Fallback when AsyncContext is not initialized: read live world time. Worker callers must go
	// through the cached AsyncContext.GetTimeSeconds() above; UWorld::GetTimeSeconds touches GT-mutable
	// timer state and is not safe from generic workers.
	check(IsInGameThread() || IsInParallelGameThread());
	return GetWorld()->GetTimeSeconds();
}

void UFastGeoContainer::OnNavigationInitDone(const UNavigationSystemBase& InNavigationSystem)
{
	const UWorld* World = GetWorld();
	const UNavigationSystemBase* NavigationSystem = World ? World->GetNavigationSystem() : nullptr;

	if (NavigationSystem && (NavigationSystem == &InNavigationSystem))
	{
		UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().RemoveAll(this);
		RegisterToNavigationSystem();
	}
}

void UFastGeoContainer::RegisterToNavigationSystem()
{
	UWorld* World = GetWorld();

	const UNavigationSystemBase* NavigationSystem = World->GetNavigationSystem();
	check(NavigationSystem);
	check(NavigationSystem->IsWorldInitDone());
	check(IsRegistered() || IsRegistering());

	if (!FNavigationSystem::SupportsDynamicChanges(World))
	{
		return;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::RegisterToNavigationSystem);
	check(NavigationElementHandles.IsEmpty());

	ForEachComponentCluster([this, World](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.ForEachComponent<FFastGeoPrimitiveComponent>([this, World](FFastGeoPrimitiveComponent& Component)
		{
			if (Component.IsNavigationRelevant())
			{
				FNavigationElement Element(*this, reinterpret_cast<const uint64>(&Component));
				Element.SetDirtyAreaOnRegistration(!Component.ShouldSkipNavigationDirtyAreaOnAddOrRemove());
				Element.SetBounds(Component.GetNavigationBounds());
				Element.SetBodySetup(Component.GetBodySetup());
				Element.SetTransform(Component.GetTransform());
				Element.SetGeometryExportType(Component.HasCustomNavigableGeometry());
				Element.NavigationDataExportDelegate.BindWeakLambda(this, [&Component](const FNavigationElement& NavigationElement, FNavigationRelevantData& OutNavigationRelevantData)
				{
					Component.GetNavigationData(OutNavigationRelevantData);
				});
				Element.CustomGeometryExportDelegate.BindWeakLambda(this, [&Component](const FNavigationElement& NavigationElement, FNavigableGeometryExport& OutGeometry, bool& bOutShouldExportDefaultGeometry)
				{
					bOutShouldExportDefaultGeometry = Component.DoCustomNavigableGeometryExport(OutGeometry);
				});
				FNavigationElementHandle Handle = FNavigationSystem::AddNavigationElement(World, MoveTemp(Element));
				if (ensure(Handle.IsValid()))
				{
					NavigationElementHandles.Add(Handle);
				}
			}
		});
	});
}

void UFastGeoContainer::UnregisterFromNavigationSystem()
{
	UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().RemoveAll(this);

	if (!NavigationElementHandles.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::UnregisterFromNavigationSystem);

		for (FNavigationElementHandle& ElementHandle : NavigationElementHandles)
		{
			FNavigationSystem::RemoveNavigationElement(GetWorld(), ElementHandle);
		}

		NavigationElementHandles.Empty();
	}
}

void UFastGeoContainer::OnCreateRenderStateBegin_GameThread()
{
	check(PendingCreate.RenderState.NumToProcess == 0);
	check(PendingCreate.RenderState.TotalNumProcessed != PendingCreate.RenderState.ComponentsToProcess.Num());

	PendingCreate.RenderState.bIsInBlockingWait = GetWorldSubsystem()->IsWaitingForCompletion();
	PendingCreate.RenderState.NumToProcess = PendingCreate.RenderState.ComponentsToProcess.Num() - PendingCreate.RenderState.TotalNumProcessed;

	// Refresh cached time (see FFastGeoDecalComponent::CreateRenderState)
	AsyncContext.UpdateTimeSeconds(GetWorld()->GetTimeSeconds());
}

void UFastGeoContainer::OnDestroyRenderStateBegin_GameThread()
{
	check(PendingDestroy.RenderState.NumToProcess == 0);
	check(PendingDestroy.RenderState.TotalNumProcessed != PendingDestroy.RenderState.ComponentsToProcess.Num());

	PendingDestroy.RenderState.bIsInBlockingWait = GetWorldSubsystem()->IsWaitingForCompletion();

	// There is no throttling for the async destruction task
	PendingDestroy.RenderState.NumToProcess = PendingDestroy.RenderState.ComponentsToProcess.Num();
}

template <bool bDestroyFirst>
void UFastGeoContainer::ExecuteCreateRenderState_Concurrent(FFastGeoRenderStateBatch& State)
{
	check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());

	static constexpr int32 MinNumElementsToProcessPerThread = 8;
	const int32 NumComponentsToProcess = State.NumToProcess;
	const int32 MaxNumThreads = State.bIsInBlockingWait ? INT32_MAX : FastGeo::Private::GAsyncRenderStateTaskParallelWorkerCount;
	const int32 NumThreads = FMath::Clamp(NumComponentsToProcess / MinNumElementsToProcessPerThread, 1, MaxNumThreads);
	const bool bIsParallelForAllowed = NumThreads > 1 && FApp::ShouldUseThreadingForPerformance();

	float AvailableTimeBudgetMS;
	int32 AvailableComponentsBudget;
	int32 TimeEpoch;

	UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
	WorldSubsystem->RequestAsyncRenderStateTasksBudget_Concurrent(AvailableTimeBudgetMS, AvailableComponentsBudget, TimeEpoch);
	
	const int32 ComponentsBudget = FMath::Min(NumComponentsToProcess, AvailableComponentsBudget);
	const double TimeBudgetSeconds = AvailableTimeBudgetMS / 1000.0;
	UE::FTimeout Timeout = UE::FTimeout(TimeBudgetSeconds);

	std::atomic<int32> NextIndex{ 0 };
	std::atomic<int32> NumProcessed{ 0 };

	if (ComponentsBudget > 0 && !Timeout.IsExpired())
	{
		ParallelFor(NumThreads, [&](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::ExecuteCreateRenderState_Concurrent);
			while(true)
			{
				// Time budget exceeded ?
				if (Timeout.IsExpired())
				{
					return; 
				}

				// All work completed ?
				const int32 LocalIndex = NextIndex.fetch_add(1, std::memory_order_relaxed);
				if (LocalIndex >= ComponentsBudget)
				{
					return; 
				}

				FFastGeoComponent* ComponentToProcess = State.ComponentsToProcess[State.TotalNumProcessed + LocalIndex];
				if constexpr (bDestroyFirst)
				{
					ComponentToProcess->DestroyRenderState(nullptr);
				}
				ComponentToProcess->CreateRenderState(nullptr);

				NumProcessed.fetch_add(1, std::memory_order_relaxed);
			}
		}, bIsParallelForAllowed ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}

	State.NumProcessed = NumProcessed.load(std::memory_order_relaxed);
	WorldSubsystem->CommitAsyncRenderStateTasksBudget_Concurrent(Timeout.GetElapsedSeconds() * 1000, State.NumProcessed, TimeEpoch);
}

// Explicit template instantiations
template void UFastGeoContainer::ExecuteCreateRenderState_Concurrent<false>(FFastGeoRenderStateBatch&);
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
template void UFastGeoContainer::ExecuteCreateRenderState_Concurrent<true>(FFastGeoRenderStateBatch&);
#endif

void UFastGeoContainer::OnCreateRenderState_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnCreateRenderState_Concurrent);
	ExecuteCreateRenderState_Concurrent<false>(PendingCreate.RenderState);
}

void UFastGeoContainer::OnDestroyRenderState_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnDestroyRenderState_Concurrent);

	check(PendingDestroy.RenderState.NumToProcess == PendingDestroy.RenderState.ComponentsToProcess.Num());
	{
		FFastGeoDestroyRenderStateContext Context(GetScene());

		for (FFastGeoComponent* Component : PendingDestroy.RenderState.ComponentsToProcess)
		{
			Component->DestroyRenderState(&Context);
		}

		PendingDestroy.RenderState.NumProcessed = PendingDestroy.RenderState.NumToProcess;
	}
}

void UFastGeoContainer::OnCreateRenderStateEnd_GameThread()
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		check(PendingCreate.RenderState.NumToProcess != 0);
		check(PendingCreate.RenderState.TotalNumProcessed != PendingCreate.RenderState.ComponentsToProcess.Num());
	}

	const int32 BatchNumProcessed = PendingCreate.RenderState.NumProcessed;
	const int32 BatchStartIndex = PendingCreate.RenderState.TotalNumProcessed;

	PendingCreate.RenderState.TotalNumProcessed += BatchNumProcessed;
	PendingCreate.RenderState.NumToProcess = 0;
	PendingCreate.RenderState.NumProcessed = 0;
	PendingCreate.RenderState.bIsInBlockingWait = false;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Collect PSO-delayed components from the batch just processed.
	// Covers both DelayUntilPSOPrecached (all component types delayed) and
	// UseFallbackMaterialUntilPSOPrecached (decals/lights delayed, static meshes use fallback).
	// Both strategies check != AlwaysCreate in their CreateSceneProxy.
	if (GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
	{
		for (int32 i = 0; i < BatchNumProcessed; ++i)
		{
			FFastGeoComponent* Comp = PendingCreate.RenderState.ComponentsToProcess[BatchStartIndex + i];
			if (Comp->IsRenderStateDelayed())
			{
				PSODeferredPendingComponents.Add(Comp);
			}
			// Safety net for UseFallbackMaterialUntilPSOPrecached: if the PSO callback fired
			// before CreateRenderState ran, the callback saw NeedsPSORecreate()=false (proxy
			// didn't exist yet) and was silently ignored. Detect this by checking for components
			// created with fallback material whose PSO has already completed.
			else if (Comp->NeedsPSORecreate() && !Comp->GetPSOPrecacheComponentData().IsPSOPrecaching())
			{
				RoutePSORecreateComponent(Comp);
			}
		}
	}
#endif

	if (!IsInitialCreateRenderStatePassCompleted())
	{
		check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());
		StartCreateRenderStateWork();
	}
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	else if (HasPSODeferredComponents())
	{
		if (GetWorldSubsystem()->IsWaitingForCompletion())
		{
			ForceCreateDeferredComponents();
		}
		else
		{
			// Normal per-frame path: park and wait for PSO callbacks.
			// As PSO callbacks fire, components move from PSODeferredPendingComponents to PSODeferredReadyComponents.
			// When the pending set is empty (or the periodic safety poll detects all are ready), 
			// CheckPSODeferredReadiness() triggers the deferred create pass.
			PSODeferredReadyComponents.Reserve(PSODeferredPendingComponents.Num());
			GetWorldSubsystem()->RegisterContainerWithPSODeferredWork(this);

			// Handle the race where PSO callbacks fired before we parked. Those callbacks
			// did nothing (component wasn't in the pending set yet). Poll IsPSOPrecaching()
			// now to detect any that already completed and move them to the ready set.
			CheckPSODeferredReadiness(/*bPollPending=*/ true);
		}
	}
#endif
	else
	{
		// Render work complete - check if all registration systems are done
		TryCompleteRegistration();
	}
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void UFastGeoContainer::OnComponentPSOPrecacheCompleted(FFastGeoComponent* Component)
{
	check(IsInGameThread());

	if (PSODeferredPendingComponents.Contains(Component))
	{
		// DelayUntilPSOPrecached: component had no proxy, PSO is now ready for deferred create
		OnPSODeferredReady(Component);
	}
	else if (Component->NeedsPSORecreate())
	{
		// UseFallbackMaterialUntilPSOPrecached: component has proxy with fallback material, PSO is now ready, swap to real material.
		RoutePSORecreateComponent(Component);
	}
	// Else: component is Delayed but not yet parked (PSO callback fired before OnCreateRenderStateEnd_GameThread collected it), 
	// or in None state (proxy failed for non-PSO reasons -- retrying won't help). 
	// CheckPSODeferredReadiness() will detect the early completion via IsPSOPrecaching() polling when End_GameThread parks.
}

void UFastGeoContainer::OnPSODeferredReady(FFastGeoComponent* Component)
{
	check(IsInGameThread());

	// Move from pending to ready. O(1) remove from TSet.
	PSODeferredPendingComponents.Remove(Component);
	PSODeferredReadyComponents.Add(Component);

	// Process ready components incrementally -- don't wait for all pending to complete.
	// CheckPSODeferredReadiness handles early-outs (nothing ready, pass in flight).
	CheckPSODeferredReadiness();
}

void UFastGeoContainer::CheckPSODeferredReadiness(bool bPollPending)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::CheckPSODeferredReadiness);
	check(IsInGameThread());

	if (PSODeferredPendingComponents.IsEmpty() && PSODeferredReadyComponents.IsEmpty())
	{
		return;
	}

	if (bPollPending)
	{
		// Safety net: detect components whose PSO callback was lost (component destroyed before
		// callback fired, lifecycle ID mismatch in FPSOPrecacheFinishedTask, etc.). Without this
		// check, a lost callback would leave the container parked forever with AddToWorld never
		// completing. We poll IsPSOPrecaching() to detect PSOs that completed without notifying us.
		// Skipped when called per-callback from OnPSODeferredReady (bPollPending=false) to avoid
		// O(n) iteration on every callback. See header comment for full list of call sites.
		for (auto It = PSODeferredPendingComponents.CreateIterator(); It; ++It)
		{
			FFastGeoComponent* Comp = *It;
			if (!Comp->GetPSOPrecacheComponentData().IsPSOPrecaching())
			{
				PSODeferredReadyComponents.Add(Comp);
				It.RemoveCurrent();
			}
		}
	}

	if (PSODeferredReadyComponents.IsEmpty())
	{
		// Nothing ready yet
		return;
	}

	if (bIsProcessingPSODeferredCreate)
	{
		// A deferred create job is still in flight. Ready components will be picked up
		// when OnDeferredCreateRenderStateEnd_GameThread calls CheckPSODeferredReadiness again.
		return;
	}

	// Process whatever is ready now, even if some components are still pending.
	// This allows incremental proxy creation as PSOs complete, utilizing the per-frame
	// time budget instead of waiting idle then creating all at once.
	check(PSODeferredCreateState.ComponentsToProcess.IsEmpty());
	PSODeferredCreateState.Reset();
	PSODeferredCreateState.ComponentsToProcess = MoveTemp(PSODeferredReadyComponents);

	// Don't unregister from ContainersWithPSODeferredWork here -- this function can be
	// called from inside the WorldSubsystem::Tick polling loop, and removing from the set
	// while iterating would invalidate the iterator. The poll loop safely removes containers
	// via It.RemoveCurrent() when HasPSODeferredComponents() returns false.
	if (RegistrationState == ERegistrationState::Registering)
	{
		StartDeferredCreateRenderStateWork();
	}
}

void UFastGeoContainer::ForceCreateDeferredComponents()
{
	check(IsInGameThread());

	// Early-out if no deferred work remains. This can happen when the drain loop
	// (TickRenderWork) inside this function or the caller's TickRenderWork already
	// completed all deferred work before we get here.
	if (!HasPSODeferredComponents())
	{
		return;
	}

	// Drain any in-flight deferred create before overwriting PSODeferredCreateState.
	// A previous deferred create pass may still be executing (budget-driven re-enqueue
	// from before the blocking wait started). TickRenderWork handles both async (queue)
	// and sync (TickDeferredCreate_Sync) modes.
	while (bIsProcessingPSODeferredCreate)
	{
		TickRenderWork(/*bWaitForCompletion=*/ true);
	}
	check(PSODeferredCreateState.ComponentsToProcess.IsEmpty());

	// Blocking path (e.g., BlockTillLevelStreamingCompleted): GT is blocked and
	// cannot pump the task graph, so PSO callbacks will never fire. Merge all
	// deferred components (pending + ready) into a force-create pass. The PSO
	// delay check is skipped via IsWaitingForCompletion() in CreateSceneProxy,
	// ensuring AddToWorld + blocking wait ends with all proxies created.
	check(PSODeferredCreateState.ComponentsToProcess.IsEmpty());
	PSODeferredCreateState.Reset();
	PSODeferredCreateState.ComponentsToProcess = MoveTemp(PSODeferredReadyComponents);
	PSODeferredCreateState.ComponentsToProcess.Append(PSODeferredPendingComponents.Array());
	PSODeferredPendingComponents.Reset();
	GetWorldSubsystem()->UnregisterContainerWithPSODeferredWork(this);
	StartDeferredCreateRenderStateWork();
}

void UFastGeoContainer::StartDeferredCreateRenderStateWork()
{
	check(IsInGameThread());
	bIsProcessingPSODeferredCreate = true;
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->PushAsyncDeferredCreateJob(this);
	}
	// Sync mode: TickRenderWork_Sync handles via TickDeferredCreate_Sync
}

void UFastGeoContainer::OnDeferredCreateRenderStateBegin_GameThread()
{
	check(IsInGameThread());
	PSODeferredCreateState.NumToProcess = PSODeferredCreateState.ComponentsToProcess.Num() - PSODeferredCreateState.TotalNumProcessed;
	PSODeferredCreateState.bIsInBlockingWait = GetWorldSubsystem()->IsWaitingForCompletion();
	AsyncContext.UpdateTimeSeconds(GetWorld()->GetTimeSeconds());
}

void UFastGeoContainer::OnDeferredCreateRenderState_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnDeferredCreateRenderState_Concurrent);
	ExecuteCreateRenderState_Concurrent<false>(PSODeferredCreateState);
}

void UFastGeoContainer::OnDeferredCreateRenderStateEnd_GameThread()
{
	check(IsInGameThread());
	PSODeferredCreateState.TotalNumProcessed += PSODeferredCreateState.NumProcessed;
	PSODeferredCreateState.NumToProcess = 0;
	PSODeferredCreateState.NumProcessed = 0;
	PSODeferredCreateState.bIsInBlockingWait = false;
	bIsProcessingPSODeferredCreate = false;

	if (!PSODeferredCreateState.IsCompleted())
	{
		// Budget exhausted
		StartDeferredCreateRenderStateWork();
	}
	else
	{
		PSODeferredCreateState.Reset();
		if (HasPSODeferredComponents())
		{
			// New deferred components arrived while the previous pass was in flight
			// (e.g., a new batch of components completed their initial create pass and
			// parked while this deferred create was executing). Process them now.
			// Poll pending to detect any whose PSO callback may have been lost.
			CheckPSODeferredReadiness(/*bPollPending=*/ true);
		}
		else
		{
			TryCompleteRegistration();
		}
	}
}

bool UFastGeoContainer::TickDeferredCreate_Sync(const UE::FTimeout& Timeout)
{
	if (PSODeferredCreateState.ComponentsToProcess.IsEmpty())
	{
		return true;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::TickDeferredCreate_Sync);
	if (PSODeferredCreateState.ComponentsToProcess.Num() > 0 && PSODeferredCreateState.NumToProcess == 0)
	{
		OnDeferredCreateRenderStateBegin_GameThread();
	}

	const bool bCompleted = FastGeo::AdvanceRenderStateBudgeted(PSODeferredCreateState, Timeout, [](FFastGeoComponent* Component)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoComponent_CreateRenderState);
		Component->CreateRenderState(nullptr);
	});

	if (bCompleted)
	{
		FastGeo::Private::ResetBatchForSyncCompletion(PSODeferredCreateState);
		OnDeferredCreateRenderStateEnd_GameThread();
	}
	return bCompleted;
}

void UFastGeoContainer::RoutePSORecreateComponent(FFastGeoComponent* Component)
{
	check(Component->NeedsPSORecreate());
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		// Async mode: push to per-container recreate queue (processed by the async job queue).
		PushPSORecreateComponent(Component);
	}
	else
	{
		// Sync mode: route through ProcessPendingRecreate (runs every frame from OnWorldPreSendAllEndOfFrameUpdates).
		Component->MarkRenderStateDirty(false);
	}
}

void UFastGeoContainer::PushPSORecreateComponent(FFastGeoComponent* Component)
{
	check(IsInGameThread());
	// Async-only path. Sync mode routes through MarkRenderStateDirty -> ProcessPendingRecreate.
	check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());

	// Don't accept recreate work during unregistration -- the destroy job will clean up all proxies.
	if (RegistrationState == ERegistrationState::Unregistering)
	{
		return;
	}

	PSORecreateComponents.Add(Component);
	if (!bIsProcessingPSORecreate)
	{
		StartRecreateRenderStateWork();
	}
}

void UFastGeoContainer::StartRecreateRenderStateWork()
{
	check(IsInGameThread());

	if (!PSORecreateState.IsCompleted())
	{
		// Continuing an in-progress pass -- merge new arrivals without resetting progress.
		// NumToProcess is recalculated in OnRecreateRenderStateBegin_GameThread from Num - TotalNumProcessed.
		check(bIsProcessingPSORecreate);
		if (!PSORecreateComponents.IsEmpty())
		{
			PSORecreateState.ComponentsToProcess.Append(PSORecreateComponents);
			PSORecreateComponents.Reset();
		}
	}
	else
	{
		// Fresh pass
		check(!bIsProcessingPSORecreate);
		bIsProcessingPSORecreate = true;
		PSORecreateState.Reset();
		PSORecreateState.ComponentsToProcess = MoveTemp(PSORecreateComponents);
	}

	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		UFastGeoWorldSubsystem* WorldSubsystem = GetWorldSubsystem();
		WorldSubsystem->RegisterContainerWithAsyncPSORecreateWork(this);
		WorldSubsystem->PushAsyncRecreateRenderStateJob(this);
	}
	// Sync mode routes through MarkRenderStateDirty -> ProcessPendingRecreate instead.
}

void UFastGeoContainer::OnRecreateRenderStateBegin_GameThread()
{
	check(IsInGameThread());
	PSORecreateState.NumToProcess = PSORecreateState.ComponentsToProcess.Num() - PSORecreateState.TotalNumProcessed;
	PSORecreateState.bIsInBlockingWait = GetWorldSubsystem()->IsWaitingForCompletion();
	AsyncContext.UpdateTimeSeconds(GetWorld()->GetTimeSeconds());
}

void UFastGeoContainer::OnRecreateRenderState_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnRecreateRenderState_Concurrent);
	ExecuteCreateRenderState_Concurrent<true>(PSORecreateState);
}

void UFastGeoContainer::OnRecreateRenderStateEnd_GameThread()
{
	check(IsInGameThread());
	check(bIsProcessingPSORecreate);
	PSORecreateState.TotalNumProcessed += PSORecreateState.NumProcessed;
	PSORecreateState.NumToProcess = 0;
	PSORecreateState.NumProcessed = 0;
	PSORecreateState.bIsInBlockingWait = false;

	// Container is being unregistered -- abort all recreate work.
	// The destroy job in the pipe will clean up all proxies.
	if (RegistrationState == ERegistrationState::Unregistering)
	{
		PSORecreateState.Reset();
		PSORecreateComponents.Reset();
		bIsProcessingPSORecreate = false;
		GetWorldSubsystem()->UnregisterContainerWithAsyncPSORecreateWork(this);
		// Recreate may have been the last pending work -- check if unregistration can complete.
		TryCompleteUnregistration();
		return;
	}

	if (!PSORecreateState.IsCompleted())
	{
		StartRecreateRenderStateWork();
	}
	else
	{
		PSORecreateState.Reset();
		bIsProcessingPSORecreate = false;

		if (!PSORecreateComponents.IsEmpty())
		{
			StartRecreateRenderStateWork();
		}
		else
		{
			GetWorldSubsystem()->UnregisterContainerWithAsyncPSORecreateWork(this);
		}
	}
}
#endif

void UFastGeoContainer::OnDestroyRenderStateEnd_GameThread()
{
	if (FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		check(PendingDestroy.RenderState.NumToProcess != 0);
		check(PendingDestroy.RenderState.TotalNumProcessed != PendingDestroy.RenderState.ComponentsToProcess.Num());
	}

	PendingDestroy.RenderState.TotalNumProcessed += PendingDestroy.RenderState.NumProcessed;
	PendingDestroy.RenderState.NumToProcess = 0;
	PendingDestroy.RenderState.NumProcessed = 0;	
	PendingDestroy.RenderState.bIsInBlockingWait = false;

	if (!IsDestroyRenderStateCompleted())
	{
		check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());
		StartDestroyRenderStateWork();
	}
	else
	{
		// Render work complete - check if all unregistration systems are done
		TryCompleteUnregistration();
	}
}

void UFastGeoContainer::BeginDestroy()
{
	Super::BeginDestroy();
	DestroyFence.BeginFence();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		ForEachComponentCluster([](FFastGeoComponentCluster& ComponentCluster)
		{
			ComponentCluster.ForEachComponent<FFastGeoPrimitiveComponent>([](FFastGeoPrimitiveComponent& Component)
			{
				Component.NotifyRenderStateChanged();
			});
		});
	}
#endif
}

bool UFastGeoContainer::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy() && DestroyFence.IsFenceComplete();
}

void UFastGeoContainer::AddComponentCluster(FFastGeoComponentCluster* ComponentCluster)
{
	FFastGeoComponentCluster* NewComponentCluster;
	if (ComponentCluster->IsA<FFastGeoHLOD>())
	{
		NewComponentCluster = &HLODs.Add_GetRef(*ComponentCluster->CastTo<FFastGeoHLOD>());
		NewComponentCluster->SetComponentClusterIndex(HLODs.Num() - 1);
	}
	else
	{
		NewComponentCluster = &ComponentClusters.Add_GetRef(*ComponentCluster);
		NewComponentCluster->SetComponentClusterIndex(ComponentClusters.Num() - 1);
	}
}

class FFastGeoGatherFastGeoContainerAssetRefsArchive : public FArchive
{
public:
	FFastGeoGatherFastGeoContainerAssetRefsArchive(UFastGeoContainer& Container)
		: FFastGeoGatherFastGeoContainerAssetRefsArchive()
	{
		Container.SerializeComponentClusters(*this);
	}

#if WITH_EDITOR
	FFastGeoGatherFastGeoContainerAssetRefsArchive(UFastGeoContainer& Container, TFunctionRef<bool(const FFastGeoComponent&)> ShouldSerializeComponent)
		: FFastGeoGatherFastGeoContainerAssetRefsArchive()
	{
		Container.SerializeComponentClustersWithFilter(*this, ShouldSerializeComponent);
	}
#endif

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		if (Obj && !Obj->IsTemplate() && !Obj->HasAnyFlags(RF_Transient))
		{
			if (!UniqueAssets.Contains(Obj))
			{
				check(IsKnownClass(Obj->GetClass()));
				UniqueAssets.Add(Obj);
			}
		}
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Obj) override
	{
		UObject* ObjPtr = Obj.Get();
		FArchive& Result = operator<<(ObjPtr);
		Obj = ObjPtr;
		return Result;
	}

	const TSet<UObject*>& GetUniqueAssets() { return UniqueAssets; }

private:
	FFastGeoGatherFastGeoContainerAssetRefsArchive()
	{
		KnownClasses.Add(UTexture::StaticClass());
		KnownClasses.Add(UTextureLightProfile::StaticClass());
		KnownClasses.Add(URuntimeVirtualTexture::StaticClass());
		KnownClasses.Add(UStaticMesh::StaticClass());
		KnownClasses.Add(UMaterialInterface::StaticClass());
		KnownClasses.Add(UPhysicalMaterial::StaticClass());
		KnownClasses.Add(USkeletalMesh::StaticClass());
		KnownClasses.Add(UTransformProviderData::StaticClass());

		SetIsPersistent(true);
		SetIsSaving(true);
		ArIgnoreOuterRef = true;
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
	}

	bool IsKnownClass(UClass* Class)
	{
		for (UClass* KnownClass : KnownClasses)
		{
			if (Class->IsChildOf(KnownClass))
			{
				return true;
			}
		}
		return false;
	}

	TSet<UClass*> KnownClasses;
	TSet<UObject*> UniqueAssets;
};

#if WITH_EDITOR
void UFastGeoContainer::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	ForEachComponentCluster([&ObjectSaveContext](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.PreSave(ObjectSaveContext);
	});

	// Collect base materials and Nanite overrides into separate sets.
	// Base materials go to hard refs. 
	// Nanite overrides go to soft refs (may be stripped on non-Nanite platforms via FMaterialOverrideNanite::Serialize).
	if (ObjectSaveContext.IsCooking())
	{
		TSet<UMaterialInterface*> BaseSet;
		TSet<UMaterialInterface*> NaniteOverrideSet;

		ForEachComponentCluster([&BaseSet, &NaniteOverrideSet](const FFastGeoComponentCluster& Cluster)
		{
			Cluster.ForEachComponent<FFastGeoComponent>([&BaseSet, &NaniteOverrideSet](const FFastGeoComponent& Comp)
			{
				Comp.ForEachMaterial([&BaseSet, &NaniteOverrideSet](UMaterialInterface* Mat, bool bIsNaniteOverride)
				{
					if (bIsNaniteOverride)
					{
						NaniteOverrideSet.Add(Mat);
					}
					else
					{
						BaseSet.Add(Mat);
					}
				});
			});
		});

		// If a Nanite override is also used directly as a base material, keep it in hard refs only
		NaniteOverrideSet = NaniteOverrideSet.Difference(BaseSet);

		MaterialAssetHardRefs.Reset(BaseSet.Num());
		for (UMaterialInterface* Mat : BaseSet)
		{
			MaterialAssetHardRefs.Add(Mat);
		}

		MaterialAssetNaniteOverrideSoftRefs.Reset(NaniteOverrideSet.Num());
		for (UMaterialInterface* Mat : NaniteOverrideSet)
		{
			MaterialAssetNaniteOverrideSoftRefs.Emplace(Mat);
		}

		// Clear transient cache (will be rebuilt at load time)
		CachedMaterialAssets.Reset();
		bMaterialAssetsCached = false;

		// Build non-render-only asset mask for runtime stripping
		// on non-rendering processes (e.g. dedicated server).
		// Uses the full serialization path (cluster/HLOD data + components) but filters
		// out render-only components. This ensures cluster/HLOD-level asset refs are kept.
		auto IsNonRenderOnly = [](const FFastGeoComponent& Comp) -> bool
		{
			const FFastGeoPrimitiveComponent* Primitive = Comp.CastTo<FFastGeoPrimitiveComponent>();
			return Primitive && (Primitive->IsCollisionEnabled() || Primitive->IsNavigationRelevant());
		};

		FFastGeoGatherFastGeoContainerAssetRefsArchive NonRenderOnlyGatherAr(*this, IsNonRenderOnly);
		const TSet<UObject*>& NonRenderOnlyAssetSet = NonRenderOnlyGatherAr.GetUniqueAssets();

		NonRenderOnlyAssetMask.Init(false, Assets.Num());
		for (int32 i = 0; i < Assets.Num(); ++i)
		{
			if (NonRenderOnlyAssetSet.Contains(Assets[i]))
			{
				NonRenderOnlyAssetMask[i] = true;
			}
		}
	}
}
#endif

void UFastGeoContainer::CollectAssetReferences()
{
	Assets = FFastGeoGatherFastGeoContainerAssetRefsArchive(*this).GetUniqueAssets().Array();
}

void UFastGeoContainer::OnCreated(bool bCollectReferences)
{
	const bool bIsGameWorld = GetWorld()->IsGameWorld();

	// Initialize component clusters & components dynamic properties
	InitializeDynamicProperties(bIsGameWorld);

	// Always collect references outside of game worlds.
	if (bCollectReferences || !bIsGameWorld)
	{
		// Collect references in order to avoid garbage collection of objects that may now be unreferenced
		// The fast geo container will hold onto those objects if necessary.
		CollectAssetReferences();
	}
}

void UFastGeoContainer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	FastGeo::Private::FAssetRemapArchive AssetRemapAr(Ar, Assets);
	SerializeComponentClusters(AssetRemapAr);
	Ar << NonRenderOnlyAssetMask;
	// Manual serialization: MaterialAssetHardRefs is intentionally non-UPROPERTY (see header)
	// to skip its duplicate UObject* traversal during GC mark. The materials remain GC-anchored
	// via Assets, but the field still needs to round-trip on save/load.
	Ar << MaterialAssetHardRefs;

#if !WITH_EDITOR
	if (Ar.IsLoading())
	{
		// On non-rendering processes (e.g. dedicated server), release render-only asset
		// references and strip render-only components to reduce memory usage.
		ConditionalStripRenderOnlyComponentsAndAssets();

		// Once loaded, initialize component clusters & components dynamic properties
		InitializeDynamicProperties(true);

		// Resolve pre-computed material hard/soft refs into one single cached array.
		// All material packages are loaded (hard-referenced via Assets).
		// Stripped materials like Nanite overrides on non-Nanite platform will be skipped.
		CacheMaterialAssets();
	}
#endif
}

void UFastGeoContainer::SerializeComponentClusters(FArchive& Ar)
{
	Ar << ComponentClusters;
	Ar << HLODs;
}

#if WITH_EDITOR
void UFastGeoContainer::SerializeComponentClustersWithFilter(FArchive& Ar, TFunctionRef<bool(const FFastGeoComponent&)> ShouldSerializeComponent)
{
	for (auto& Cluster : ComponentClusters)
	{
		Cluster.SerializeWithComponentFilter(Ar, ShouldSerializeComponent);
	}
	for (auto& HLOD : HLODs)
	{
		HLOD.SerializeWithComponentFilter(Ar, ShouldSerializeComponent);
	}
}
#endif

#if !WITH_EDITOR
void UFastGeoContainer::ConditionalStripRenderOnlyComponentsAndAssets()
{
	if (bHasStrippedRenderOnlyComponents)
	{
		return;
	}

	bHasStrippedRenderOnlyComponents = true;

	if (FApp::CanEverRender() || !FastGeo::Private::GStripRenderOnlyComponentsOnNonRenderingProcess)
	{
		// Rendering process -- discard the extra cook data, keep all assets
		NonRenderOnlyAssetMask.Empty();
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::ConditionalStripRenderOnlyComponentsAndAssets);

	// Null out Assets entries not needed by non-render-only components.
	// Bits are set for entries to keep.
	check(NonRenderOnlyAssetMask.Num() == Assets.Num());
	for (int32 i = 0; i < Assets.Num(); ++i)
	{
		if (!NonRenderOnlyAssetMask[i])
		{
			Assets[i] = nullptr;
		}
	}
	NonRenderOnlyAssetMask.Empty();

	// Materials are purely render-related -- not needed on non-rendering processes
	MaterialAssetHardRefs.Empty();
	MaterialAssetNaniteOverrideSoftRefs.Empty();

	// Strip render-only components
	int32 TotalStripped = 0;
	ForEachComponentCluster([&TotalStripped](FFastGeoComponentCluster& ComponentCluster)
	{
		TotalStripped += ComponentCluster.StripRenderOnlyComponents();
	});

	UE_CLOGF(TotalStripped > 0, LogFastGeoStreaming, Verbose, "FastGeo: Stripped %d render-only components in '%ls'", TotalStripped, *GetName());
}
#endif

void UFastGeoContainer::OnPrecachePSOsBegin_GameThread()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnPrecachePSOsBegin_GameThread);
	check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());
	check(IsComponentPSOPrecachingEnabled());

	// PSO precaching can run before StartRegisterTransition, so initialize all prerequisites now.
	InitializePrerequisites();

	// Track async PSO precaching as pending work so that HasAnyPendingTasks() returns true.
	// This ensures that WaitForAllPendingWorkCompletion properly gates on async PSO work completion.
	// Set TotalNumProcessed to -1 so IsCompleted() (TotalNumProcessed >= ComponentsToProcess.Num())
	// returns false while the async job is in flight.
	PendingCreate.PrecachePSOState.TotalNumProcessed = -1;
#endif
}

void UFastGeoContainer::OnPrecachePSOs_Concurrent()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnPrecachePSOs_Concurrent);
	check(FFastGeoStreamingModule::IsAsyncRenderWorkAllowed());
	check(IsComponentPSOPrecachingEnabled());

	ForEachComponentCluster([](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.ForEachComponent<FFastGeoComponent>([](FFastGeoComponent& Component)
		{
			Component.PrecachePSOs_Concurrent();
		});
	});
#endif
}

void UFastGeoContainer::OnPrecachePSOsEnd_GameThread()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Clear the pending PSO precache state set by OnPrecachePSOsBegin_GameThread
	// so that HasAnyPendingTasks() no longer reports PSO work as in-flight.
	PendingCreate.PrecachePSOState.Reset();
#endif
}

void UFastGeoContainer::PrecachePSOs()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// This function is currently only starting precaching when running in async mode
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::PrecachePSOs_GameThread);
	StartPrecachePSOWork();
	// Recreate is unrelated to PSO precache setup.
	if (HasAnyPendingRegistrationTasks())
	{
		// Only TickRenderWork is needed here to Launch the PSO precache job via ProcessAsyncRenderStateJobs.
		constexpr bool bWaitForCompletion = false;
		TickRenderWork(bWaitForCompletion);
	}
#endif
}

void UFastGeoContainer::ForEachMaterialAsset(TFunctionRef<void(UMaterialInterface*)> Func)
{
	// Cooked builds: already resolved from soft refs in Serialize (ALT).
	// PIE / Transient FastGeoContainer: build from component walk on first call (runs once on GT).
	if (!bMaterialAssetsCached)
	{
		BuildMaterialAssets();
	}

	for (UMaterialInterface* Mat : CachedMaterialAssets)
	{
		Func(Mat);
	}
}

void UFastGeoContainer::BuildMaterialAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::BuildMaterialAssets);

	TSet<UMaterialInterface*> UniqueSet;

	ForEachComponentCluster([&UniqueSet](const FFastGeoComponentCluster& Cluster)
	{
		Cluster.ForEachComponent<FFastGeoComponent>([&UniqueSet](const FFastGeoComponent& Comp)
		{
			Comp.ForEachMaterial([&UniqueSet](UMaterialInterface* Mat, bool /*bIsNaniteOverride*/)
			{
				UniqueSet.Add(Mat);
			});
		});
	});

	CachedMaterialAssets = UniqueSet.Array();
	bMaterialAssetsCached = true;
}

#if !WITH_EDITOR
void UFastGeoContainer::CacheMaterialAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::CacheMaterialAssets);

	CachedMaterialAssets = MaterialAssetHardRefs;
	CachedMaterialAssets.Reserve(CachedMaterialAssets.Num() + MaterialAssetNaniteOverrideSoftRefs.Num());

	// Resolve Nanite overrides, stripped materials (e.g. Nanite override on non-Nanite platform) resolve to null and are skipped
	for (const TSoftObjectPtr<UMaterialInterface>& SoftRef : MaterialAssetNaniteOverrideSoftRefs)
	{
		if (UMaterialInterface* Mat = SoftRef.Get())
		{
			CachedMaterialAssets.Add(Mat);
		}
	}

	bMaterialAssetsCached = true;
}
#endif

/**
 * Cache world/scene/physics/subsystem pointers for worker-thread access.
 * Workers read cached pointers instead of performing lookups not safe from
 * generic workers (e.g., subsystem collection access, world time reads).
 */
void FFastGeoAsyncContext::Initialize(UFastGeoContainer& Container)
{
	if (bInitialized)
	{
		return;
	}

	check(IsInGameThread());

	World = Container.GetWorld();
	check(World);
	Scene = World->Scene;
	check(Scene);
	PhysicsScene = World->GetPhysicsScene();
	check(PhysicsScene);
	WorldSubsystem = World->GetSubsystemChecked<UFastGeoWorldSubsystem>();
	TimeSeconds = World->GetTimeSeconds();
	bInitialized = true;
}

/**
 * Pre-call ConditionalPostLoad on all materials referenced by this container.
 * This is necessary because we don't want any of these function to call
 * ConditionalPostLoad on materials from worker threads:
 *     - UMaterialInstance::PrecachePSOs
 *     - FStaticMeshComponentHelper::GetMaterial
 *     - FStaticMeshComponentHelper::GetUsedRayTracingOnlyMaterials
 * 
 * Note that once the preemptive ConditionalPostLoad is removed from these functions,
 * this prerequisite step can be removed.
 */
void FFastGeoMaterialPostLoadInit::Initialize(UFastGeoContainer& Container)
{
	if (bInitialized)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoMaterialPostLoadInit::Initialize);
	check(IsInGameThread());

	Container.ForEachMaterialAsset([](UMaterialInterface* Mat)
	{
		if (Mat->HasAnyFlags(RF_NeedPostLoad))
		{
			Mat->ConditionalPostLoad();
		}
	});

	bInitialized = true;
}

/**
 * Warm all materials relevance cache because so that FastGeo 
 * workers call to functions below will the thread safe:
 *     - FMeshComponentHelper::GetMaterialRelevance
 *     - FNaniteResourcesHelper::AuditMaterials
 *     - FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl
 */
void FFastGeoMaterialRelevanceInit::Initialize(UFastGeoContainer& Container)
{
#if !WITH_EDITOR
	if (bInitialized || !FFastGeoStreamingModule::IsAsyncRenderWorkAllowed())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoMaterialRelevanceInit::Initialize);
	check(IsInGameThread());

	const EShaderPlatform Platform = Container.GetWorld()->Scene->GetShaderPlatform();

	Container.ForEachMaterialAsset([Platform](UMaterialInterface* Mat)
	{
		Mat->WarmRelevanceCache(Platform);
	});

	// Also pre-warm default material on the GameThread (UMaterial::GetDefaultMaterial lazy init has check(IsInGameThread())).
	if (UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface))
	{
		DefaultMaterial->WarmRelevanceCache(Platform);
	}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Also pre-warm PSO fallback material
	if (UMaterialInterface* PSOFallbackMaterial = UPSOPrecacheSettingsManager::GetFallbackMaterial())
	{
		PSOFallbackMaterial->WarmRelevanceCache(Platform);
	}
#endif

	bInitialized = true;
#endif
}

void UFastGeoContainer::InitializePrerequisites()
{
	check(IsInGameThread());

	AsyncContext.Initialize(*this);
	MaterialPostLoadInit.Initialize(*this);
	MaterialRelevanceInit.Initialize(*this);
}

void UFastGeoContainer::InitializeDynamicProperties(bool bInitForPlay)
{
	ForEachComponentCluster([this](FFastGeoComponentCluster& ComponentCluster)
	{
		ComponentCluster.SetOwnerContainer(this);
		ComponentCluster.InitializeDynamicProperties();
	});

	if (bInitForPlay)
	{
		PrimitiveComponents.Reset();
		StaticMeshComponents.Reset();
		CollisionComponents.Reset();
		ForEachComponentCluster([this](FFastGeoComponentCluster& ComponentCluster)
		{
			ComponentCluster.ForEachComponent([this](FFastGeoComponent& Component)
			{
				if (Component.IsCollisionEnabled())
				{
					CollisionComponents.Add(&Component);
				}

				if (FFastGeoPrimitiveComponent* PrimitiveComponent = Component.CastTo<FFastGeoPrimitiveComponent>())
				{
					PrimitiveComponents.Add(PrimitiveComponent);
				}
				if (FFastGeoStaticMeshComponent* StaticMeshComponent = Component.CastTo<FFastGeoStaticMeshComponent>())
				{
					StaticMeshComponents.Add(StaticMeshComponent);
				}
			});
		});
	}
}

void UFastGeoContainer::OnCreatePhysicsStateBegin_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnCreatePhysicsStateBegin_GameThread);

	if (!CollisionComponents.IsEmpty())
	{
		check(FPhysScene::SupportsAsyncPhysicsStateCreation());

		PendingCreate.PhysicsState.ComponentsToProcess = &CollisionComponents;
		check(PendingCreate.PhysicsState.TotalNumProcessed == 0);

		UWorld* World = GetWorld();
		check(World);
		FPhysScene* PhysScene = GetPhysicsScene();
		check(PhysScene);
		PhysScene->PushAsyncCreatePhysicsStateJob(this);
	}
}

void UFastGeoContainer::OnDestroyPhysicsStateBegin_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnDestroyPhysicsStateBegin_GameThread);

	if (!CollisionComponents.IsEmpty())
	{
		check(FPhysScene::SupportsAsyncPhysicsStateDestruction());

		PendingDestroy.PhysicsState.ComponentsToProcess = &CollisionComponents;
		check(PendingDestroy.PhysicsState.TotalNumProcessed == 0);

		UWorld* World = GetWorld();
		check(World);
		FPhysScene* PhysScene = GetPhysicsScene();
		check(PhysScene);
		verify(PhysScene->PushAsyncDestroyPhysicsStateJob(this));
	}
}

bool UFastGeoContainer::AllowsAsyncPhysicsStateCreation() const
{
	check(FPhysScene::SupportsAsyncPhysicsStateCreation());
	return true;
}

bool UFastGeoContainer::AllowsAsyncPhysicsStateDestruction() const
{
	check(FPhysScene::SupportsAsyncPhysicsStateDestruction());
	return true;
}

bool UFastGeoContainer::IsAsyncPhysicsStateCreated() const
{
	return PhysicsState == EPhysicsState::Created;
}

UObject* UFastGeoContainer::GetAsyncPhysicsStateObject() const
{
	return const_cast<UFastGeoContainer*>(this);
}

void UFastGeoContainer::OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncCreatePhysicsStateBegin_GameThread);
	check(PhysicsState == EPhysicsState::None);
	check(PendingCreate.PhysicsState.ComponentsToProcess);
	check(PendingCreate.PhysicsState.ComponentsToProcess->Num());
	PhysicsState = EPhysicsState::Creating;

	// No need to fill OutRootedObjects as Assets already keeps objects alive
	bool bSingleThreaded = !FApp::ShouldUseThreadingForPerformance();
	ParallelFor(CollisionComponents.Num(), [&](int32 Index)
	{
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);
		CollisionComponents[Index]->OnAsyncCreatePhysicsStateBegin_GameThread();
	}, bSingleThreaded);
}

bool UFastGeoContainer::OnAsyncCreatePhysicsState(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncCreatePhysicsState);
	check((PhysicsState == EPhysicsState::Creating) && !PendingCreate.PhysicsState.IsCompleted());
	
	for (int Index = PendingCreate.PhysicsState.TotalNumProcessed; Index < PendingCreate.PhysicsState.ComponentsToProcess->Num(); ++Index)
	{
		FFastGeoComponent* Component = (*PendingCreate.PhysicsState.ComponentsToProcess)[Index];
		Component->OnAsyncCreatePhysicsState();
		++PendingCreate.PhysicsState.TotalNumProcessed;
		if (!PendingCreate.PhysicsState.IsCompleted() && Timeout.IsExpired())
		{
			return false;
		}
	}

	return true;
}

void UFastGeoContainer::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncCreatePhysicsStateEnd_GameThread);
	check(PendingCreate.PhysicsState.ComponentsToProcess && PendingCreate.PhysicsState.IsCompleted());
	PendingCreate.PhysicsState.Reset();
	check(PhysicsState == EPhysicsState::Creating);
	PhysicsState = EPhysicsState::Created;
	
	for (FFastGeoComponent* Component : CollisionComponents)
	{
		Component->OnAsyncCreatePhysicsStateEnd_GameThread();
	}

	RegisterBodyInstances();

	// Physics work complete - check if all registration systems are done
	TryCompleteRegistration();
}

void UFastGeoContainer::OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncDestroyPhysicsStateBegin_GameThread);
	check(PhysicsState == EPhysicsState::Created);
	check(PendingDestroy.PhysicsState.ComponentsToProcess);
	check(PendingDestroy.PhysicsState.ComponentsToProcess->Num());
	PhysicsState = EPhysicsState::Destroying;

	for (FFastGeoComponent* Component : CollisionComponents)
	{
		Component->OnAsyncDestroyPhysicsStateBegin_GameThread();
	}

	UnregisterBodyInstances();
}

bool UFastGeoContainer::OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncDestroyPhysicsState);

	for (int Index = PendingDestroy.PhysicsState.TotalNumProcessed; Index < PendingDestroy.PhysicsState.ComponentsToProcess->Num(); ++Index)
	{
		FFastGeoComponent* Component = (*PendingDestroy.PhysicsState.ComponentsToProcess)[Index];
		Component->OnAsyncDestroyPhysicsState();
		++PendingDestroy.PhysicsState.TotalNumProcessed;
		if (!PendingDestroy.PhysicsState.IsCompleted() && Timeout.IsExpired())
		{
			return false;
		}
	}
	return true;
}

void UFastGeoContainer::OnAsyncDestroyPhysicsStateEnd_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoContainer::OnAsyncDestroyPhysicsStateEnd_GameThread);
	check(PendingDestroy.PhysicsState.ComponentsToProcess && PendingDestroy.PhysicsState.IsCompleted());
	PendingDestroy.PhysicsState.Reset();
	check(PhysicsState == EPhysicsState::Destroying);
	PhysicsState = EPhysicsState::None;

	for (FFastGeoComponent* Component : CollisionComponents)
	{
		Component->OnAsyncDestroyPhysicsStateEnd_GameThread();
	}

	// Physics work complete - check if all unregistration systems are done
	TryCompleteUnregistration();
}

void UFastGeoContainer::CollectBodySetupsWithPhysicsMeshesToCreate(TSet<UBodySetup*>& OutBodySetups) const
{
	for (FFastGeoComponent* Component : CollisionComponents)
	{
		if (UBodySetup* BodySetup = const_cast<FFastGeoComponent*>(Component)->GetBodySetup(); BodySetup && !BodySetup->bCreatedPhysicsMeshes)
		{
			OutBodySetups.Add(BodySetup);
		}
	}
}

IPhysicsBodyInstanceOwner* UFastGeoContainer::ResolvePhysicsBodyInstanceOwner(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject)
	{
		FLockedReadPhysicsObjectExternalInterface PhysicsObjectInterface = FPhysicsObjectExternalInterface::LockRead(PhysicsObject);
		FChaosUserDefinedEntity* UserDefinedEntity = PhysicsObjectInterface->GetUserDefinedEntity(PhysicsObject);
		return FFastGeoPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwner(UserDefinedEntity);
	}
	return nullptr;
}

bool UFastGeoContainer::IsUsingSurrogateComponents() const
{
	return IsValid(SurrogateActor) && UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents();
}

// Worker-thread contract: SurrogateActor is set once via UPROPERTY serialization at PostLoad
// and is never reassigned at runtime. The async physics-state worker reads this pointer (and
// dereferences AFastGeoSurrogateActor::SurrogateComponents, which is also set at level load
// and not mutated at runtime) during OnAsyncCreatePhysicsState. Safety relies on the registered
// lifetime drain: WaitForAllPendingWorkCompletion ensures no physics worker is in flight when
// GT marks the surrogate actor as garbage during teardown. If a future change introduces a
// runtime mutation of SurrogateActor (or of SurrogateComponents on the actor), this getter
// stops being worker-safe and the value must be cached in FFastGeoAsyncContext on GT before
// dispatching the worker.
AFastGeoSurrogateActor* UFastGeoContainer::GetSurrogateActor() const
{
	return SurrogateActor;
}

void UFastGeoContainer::RegisterBodyInstances()
{
	if (IsUsingSurrogateComponents())
	{
		TMap<UFastGeoSurrogateComponent*, TArray<FBodyInstance*>> SurrogateBatches;
		for (FFastGeoComponent* Component : CollisionComponents)
		{
			if (FFastGeoPrimitiveComponent* PrimComp = Component->CastTo<FFastGeoPrimitiveComponent>())
			{
				if (UFastGeoSurrogateComponent* SurrogateComp = SurrogateActor->GetSurrogateComponent(PrimComp->SurrogateComponentDescriptorIndex))
				{
					TArray<FBodyInstance*>& Batch = SurrogateBatches.FindOrAdd(SurrogateComp);
					PrimComp->GetBodyInstances(Batch);
				}
			}
		}
		for (TPair<UFastGeoSurrogateComponent*, TArray<FBodyInstance*>>& Pair : SurrogateBatches)
		{
			UFastGeoSurrogateComponent* SurrogateComponent = Pair.Key;
			SurrogateComponent->RegisterBodyInstances(Pair.Value);
		}
	}
}

void UFastGeoContainer::UnregisterBodyInstances()
{
	if (IsUsingSurrogateComponents())
	{
		SurrogateActor->ForEachComponent<UFastGeoSurrogateComponent>(false, [](UFastGeoSurrogateComponent* SurrogateComponent)
		{
			SurrogateComponent->UnregisterBodyInstances();
		});
	}
}