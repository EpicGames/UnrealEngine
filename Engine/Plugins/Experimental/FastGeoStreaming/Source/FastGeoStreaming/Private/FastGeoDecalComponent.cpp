// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoDecalComponent.h"
#include "FastGeoContainer.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoRegisteredComponent.h"
#include "FastGeoDestroyRenderStateContext.h"
#include "FastGeoPSOWeakPtrTraits.h"
#include "Components/DecalComponent.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "Materials/MaterialInterface.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "SceneInterface.h"

const FFastGeoElementType FFastGeoDecalComponent::Type(&FFastGeoComponent::Type);

FFastGeoDecalComponent::FFastGeoDecalComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
	, DecalSize(FVector::ZeroVector)
	, bIsVisible(true)
	, SceneProxy(nullptr)
{
}

FFastGeoDecalComponent::FFastGeoDecalComponent(const FFastGeoDecalComponent& Other)
	: Super(Other)
	, DecalSize(Other.DecalSize)
	, SceneProxyDesc(Other.SceneProxyDesc)
	, bIsVisible(Other.bIsVisible)
	, bDestroyOwnerAfterFade(Other.bDestroyOwnerAfterFade)
	, bFadeCompleted(false)
	, SceneProxy(nullptr)
{
}

#if WITH_EDITOR
void FFastGeoDecalComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UDecalComponent* DecalComponent = CastChecked<UDecalComponent>(Component);
	SceneProxyDesc = FDeferredDecalSceneProxyDesc(DecalComponent);
	DecalSize = DecalComponent->DecalSize;
	bIsVisible = SceneProxyDesc.bDrawInGame;
	bDestroyOwnerAfterFade = DecalComponent->bDestroyOwnerAfterFade;
	check(SceneProxyDesc.Bounds == CalcBounds());
}
#endif

void FFastGeoDecalComponent::ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const
{
	if (SceneProxyDesc.DecalMaterial)
	{
		Func(SceneProxyDesc.DecalMaterial, false);
	}
}

void FFastGeoDecalComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << DecalSize;
	FArchive_Serialize_BitfieldBool(Ar, bIsVisible);
	FArchive_Serialize_BitfieldBool(Ar, bDestroyOwnerAfterFade);

	// Serialize persistent data from FDeferredDecalSceneProxyDesc
	Ar << SceneProxyDesc.DecalMaterial;
	Ar << SceneProxyDesc.TransformWithDecalScale;
	Ar << SceneProxyDesc.Bounds;
	Ar << SceneProxyDesc.DecalColor;
	Ar << SceneProxyDesc.InitializationWorldTimeSeconds;
	Ar << SceneProxyDesc.FadeScreenSize;
	Ar << SceneProxyDesc.FadeDuration;
	Ar << SceneProxyDesc.FadeStartDelay;
	Ar << SceneProxyDesc.FadeInDuration;
	Ar << SceneProxyDesc.FadeInStartDelay;
	Ar << SceneProxyDesc.SortOrder;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bDrawInGame);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bDrawInEditor);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bShouldFade);
}

void FFastGeoDecalComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	Super::ApplyWorldTransform(InTransform);

	SceneProxyDesc.Bounds = CalcBounds();
	SceneProxyDesc.TransformWithDecalScale = GetTransformIncludingDecalSize();
}

void FFastGeoDecalComponent::UpdateVisibility()
{
	UpdateVisibilityInternal();
	MarkRenderStateDirty(true);
}

void FFastGeoDecalComponent::UpdateVisibilityInternal()
{
	SceneProxyDesc.bDrawInGame = bIsVisible && GetOwnerComponentCluster()->IsVisible();
#if !UE_BUILD_SHIPPING
	SceneProxyDesc.bDrawInGame = SceneProxyDesc.bDrawInGame && UFastGeoWorldSubsystem::IsFastGeoVisible();
#endif
}

bool FFastGeoDecalComponent::ShouldComponentAddToRenderScene() const
{
	return Super::ShouldComponentAddToRenderScene() && SceneProxyDesc.bDrawInGame && !bFadeCompleted;
}

void FFastGeoDecalComponent::CreateRenderState(FRegisterComponentContext* Context)
{
	if (!ShouldComponentAddToRenderScene())
	{
		ProxyState = EProxyCreationState::None;
		return;
	}

	FWriteScopeLock WriteLock(*Lock.Get());
	check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
	check(!SceneProxy);
	ProxyState = EProxyCreationState::Creating;
	bRenderStateDirty = false;

	FSceneInterface* Scene = GetScene();
	check(Scene);

	ESceneProxyCreationError Error;
	SceneProxy = CreateSceneProxy(Error);
	if (SceneProxy)
	{
		const float SpawnTime = GetWorldTimeSeconds();
		TArray<FDeferredDecalUpdateParams> DecalsToAdd;
		DecalsToAdd.Add(
		{
			.Transform = SceneProxy->ComponentTrans,
			.DecalProxy = SceneProxy,
			.Bounds = SceneProxy->GetBounds(),
			.AbsSpawnTime = SpawnTime,
			.FadeDuration = SceneProxyDesc.FadeDuration,
			.FadeStartDelay = SceneProxyDesc.FadeStartDelay,
			.FadeInDuration = SceneProxyDesc.FadeInDuration,
			.FadeInStartDelay = SceneProxyDesc.FadeInStartDelay,
			.FadeScreenSize = SceneProxyDesc.FadeScreenSize,
			.SortOrder = SceneProxyDesc.SortOrder,
			.DecalColor = SceneProxyDesc.DecalColor,
			.OperationType = FDeferredDecalUpdateParams::EOperationType::AddToSceneAndUpdate,
		});
		Scene->BatchUpdateDecals(MoveTemp(DecalsToAdd));

		ProxyState = EProxyCreationState::Created;

		const float LifeSpan = SceneProxyDesc.FadeStartDelay + SceneProxyDesc.FadeDuration;
		if (LifeSpan > 0)
		{
			AsyncTask(ENamedThreads::GameThread, [Component = FFastGeoRegisteredComponent(this), LifeSpan]()
			{
				if (FFastGeoDecalComponent* DecalComponent = Component.TryGetRegistered<FFastGeoDecalComponent>())
				{
					DecalComponent->SetLifeSpan(LifeSpan);
				}
			});
		}
	}
	else if (Error == ESceneProxyCreationError::WaitingPSOs)
	{
		ProxyState = EProxyCreationState::Delayed;
	}
	else
	{
		ProxyState = EProxyCreationState::None;
	}
}

FDeferredDecalProxy* FFastGeoDecalComponent::CreateSceneProxy(ESceneProxyCreationError& OutError)
{
	check(!SceneProxy);

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Skip PSO delay during blocking wait -- GT cannot pump tasks so PSO callbacks won't fire.
	const bool bForceCreate = GetOwnerContainer()->GetWorldSubsystem()->IsWaitingForCompletion();
	if (PSOPrecacheComponentData.IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate && !bForceCreate)
	{
		OutError = ESceneProxyCreationError::WaitingPSOs;
		return nullptr;
	}
#endif

	OutError = ESceneProxyCreationError::None;
	InitializeSceneProxyDescDynamicProperties();
	SceneProxy = new FDeferredDecalProxy(SceneProxyDesc);
	return SceneProxy;
}

void FFastGeoDecalComponent::InitializeSceneProxyDescDynamicProperties()
{
	UpdateVisibilityInternal();
}

void FFastGeoDecalComponent::DestroyRenderState(FFastGeoDestroyRenderStateContext* Context)
{
	FWriteScopeLock WriteLock(*Lock.Get());
	if (SceneProxy)
	{
		FSceneInterface* Scene = GetScene();
		check(Scene);
		check(ProxyState == EProxyCreationState::Created);

		FFastGeoDestroyRenderStateContext::DestroyProxy(Context, SceneProxy, Scene);
		
		SceneProxy = nullptr;
		ProxyState = EProxyCreationState::None;
		bRenderStateDirty = false;
	}
	else
	{
		check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
		ProxyState = EProxyCreationState::None;
	}
}

FBoxSphereBounds FFastGeoDecalComponent::CalcBounds() const
{
	return FBoxSphereBounds(FVector(0, 0, 0), DecalSize, DecalSize.Size()).TransformBy(WorldTransform);
}

FTransform FFastGeoDecalComponent::GetTransformIncludingDecalSize() const
{
	FTransform Ret = WorldTransform;
	Ret.SetScale3D(Ret.GetScale3D() * DecalSize);
	return Ret;
}

void FFastGeoDecalComponent::SetLifeSpan(const float InLifeSpan)
{
	check(IsInGameThread());
	if (TimerHandle_DestroyDecalComponent.IsValid())
	{
		GetWorld()->GetTimerManager().ClearTimer(TimerHandle_DestroyDecalComponent);
	}
	if (InLifeSpan > 0.f)
	{
		FFastGeoRegisteredComponent Component(this);
		GetWorld()->GetTimerManager().SetTimer(TimerHandle_DestroyDecalComponent, FTimerDelegate::CreateLambda([Component]()
		{
			if (FFastGeoDecalComponent* DecalComponent = Component.TryGetRegistered<FFastGeoDecalComponent>())
			{
				if (DecalComponent->bDestroyOwnerAfterFade)
				{
					DecalComponent->bFadeCompleted = true;
				}
				DecalComponent->DestroyRenderState(nullptr);
			}
		}), InLifeSpan, false);
	}
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
bool FFastGeoDecalComponent::ShouldPrecachePSOs() const
{
	return Super::ShouldPrecachePSOs() && SceneProxyDesc.DecalMaterial && !SceneProxyDesc.DecalMaterial->HasAnyFlags(RF_NeedPostLoad);
}

void FFastGeoDecalComponent::PrecachePSOs_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoDecalComponent::PrecachePSOs_Concurrent);
	if (!ShouldPrecachePSOs())
	{
		return;
	}

	FPSOPrecacheParams PSOPrecacheParams;
	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
	VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));

	// Immediately create at high priority and thus doesn't need boosting anymore
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	FGraphEventArray PSOPrecacheCompileEvents = SceneProxyDesc.DecalMaterial->PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);

	auto OnPrecacheFinishedGT = [WeakComponent = FWeakFastGeoComponent(this)]()
	{
		if (FFastGeoDecalComponent* Component = WeakComponent.Get<FFastGeoDecalComponent>())
		{
			if (UFastGeoContainer* Container = Component->GetOwnerContainer())
			{
				if (Container->GetWorld())
				{
					Container->OnComponentPSOPrecacheCompleted(Component);
				}
			}
		}
	};

	PSOPrecacheComponentData.SetMaterialPSOPrecacheData(this, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents, OnPrecacheFinishedGT);
}
#endif