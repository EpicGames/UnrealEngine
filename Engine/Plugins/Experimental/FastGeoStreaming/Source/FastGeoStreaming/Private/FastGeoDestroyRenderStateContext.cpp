// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoDestroyRenderStateContext.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"

FFastGeoDestroyRenderStateContext::FFastGeoDestroyRenderStateContext(FSceneInterface* InScene)
	: Scene(InScene)
{
}

FFastGeoDestroyRenderStateContext::~FFastGeoDestroyRenderStateContext()
{
	if (!PrimitiveSceneProxies.IsEmpty())
	{
		Scene->BatchRemovePrimitives(MoveTemp(PrimitiveSceneProxies));
	}
	if (!DeferredDecalProxies.IsEmpty())
	{
		TArray<FDeferredDecalUpdateParams> BatchParams;
		BatchParams.Reserve(DeferredDecalProxies.Num());
		for (FDeferredDecalProxy* Proxy : DeferredDecalProxies)
		{
			BatchParams.Add(
			{
				.DecalProxy = Proxy,
				.OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete
			});
		}
		Scene->BatchUpdateDecals(MoveTemp(BatchParams));
	}
}

void FFastGeoDestroyRenderStateContext::DestroyProxy(FFastGeoDestroyRenderStateContext* InContext, FPrimitiveSceneProxy* InPrimitiveSceneProxy, FSceneInterface* InScene)
{
	check(InScene);
	check(InPrimitiveSceneProxy);

	if (InContext)
	{
		InContext->PrimitiveSceneProxies.Add(InPrimitiveSceneProxy);
	}
	else
	{
		InScene->BatchRemovePrimitives({ InPrimitiveSceneProxy });
	}
}

void FFastGeoDestroyRenderStateContext::DestroyProxy(FFastGeoDestroyRenderStateContext* InContext, FDeferredDecalProxy* InDeferredDecalProxy, FSceneInterface* InScene)
{
	check(InScene);
	check(InDeferredDecalProxy);

	if (InContext)
	{
		InContext->DeferredDecalProxies.Add(InDeferredDecalProxy);
	}
	else
	{
		TArray<FDeferredDecalUpdateParams> BatchParams;
		BatchParams.Add(
		{
			.DecalProxy = InDeferredDecalProxy,
			.OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete
		});
		InScene->BatchUpdateDecals(MoveTemp(BatchParams));
	}
}