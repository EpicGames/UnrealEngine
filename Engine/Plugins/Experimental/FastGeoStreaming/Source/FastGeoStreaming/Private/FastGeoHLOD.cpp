// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoHLOD.h"
#include "FastGeoContainer.h"
#include "FastGeoComponent.h"
#include "FastGeoStaticMeshComponent.h"
#include "FastGeoInstancedStaticMeshComponent.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "PhysicsEngine/BodySetup.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"

const FFastGeoElementType FFastGeoHLOD::Type(&FFastGeoComponentCluster::Type);

FFastGeoHLOD::FFastGeoHLOD(UFastGeoContainer* InOwner, const FName InName, FFastGeoElementType InType)
	: Super(InOwner, InName, InType)
	, bIsVisible(true)
	, bRequireWarmup(false)
{
}

void FFastGeoHLOD::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << bRequireWarmup;
	Ar << SourceCellGuid;
	Ar << HLODGuid;
	Ar << HLODGuidType;
}

ULevel* FFastGeoHLOD::GetHLODLevel() const
{
	return GetLevel();
}

FString FFastGeoHLOD::GetHLODNameOrLabel() const
{
	return Name;
}

void FFastGeoHLOD::SetVisibility(bool bInIsVisible)
{
	if (bIsVisible != bInIsVisible)
	{
		bIsVisible = bInIsVisible;
		UpdateVisibility();
	}
}

bool FFastGeoHLOD::DoesRequireWarmup() const
{
	return bRequireWarmup;
}

TSet<UObject*> FFastGeoHLOD::GetAssetsToWarmup() const
{
	TSet<UObject*> AssetsToWarmup;

	ForEachComponent<FFastGeoStaticMeshComponentBase>([&AssetsToWarmup](const FFastGeoStaticMeshComponentBase& SMC)
	{
		// Assume ISM HLOD don't need warmup, as they are actually found in the source level
		if (SMC.IsA<FFastGeoInstancedStaticMeshComponent>())
		{
			return;
		}

		for (int32 iMaterialIndex = 0; iMaterialIndex < SMC.GetNumMaterials(); ++iMaterialIndex)
		{
			if (UMaterialInterface* Material = SMC.GetMaterial(iMaterialIndex))
			{
				AssetsToWarmup.Add(Material);
			}
		}

		if (UStaticMesh* StaticMesh = SMC.GetStaticMesh())
		{
			AssetsToWarmup.Add(StaticMesh);
		}
	});

	return AssetsToWarmup;
}

const FGuid& FFastGeoHLOD::GetSourceCellGuid() const
{
	// When no source cell guid was set, try resolving it through its associated world partition runtime cell
	// This is necessary for any HLOD actor part of a level that is instanced multiple times (shared amongst multiple cells)
	if (!SourceCellGuid.IsValid())
	{
		const UWorldPartitionRuntimeCell* Cell = Cast<UWorldPartitionRuntimeCell>(GetLevel()->GetWorldPartitionRuntimeCell());
		if (Cell && Cell->GetIsHLOD())
		{
			const_cast<FFastGeoHLOD*>(this)->SourceCellGuid = Cell->GetSourceCellGuid();
		}
	}
	return SourceCellGuid;
}

bool FFastGeoHLOD::IsStandalone() const
{
	return GetStandaloneHLODGuid().IsValid();
}

const FGuid& FFastGeoHLOD::GetStandaloneHLODGuid() const
{
	return GetHLODGuid(EHLODGuidType::Standalone);
}

bool FFastGeoHLOD::IsCustomHLOD() const
{
	return GetCustomHLODGuid().IsValid();
}

const FGuid& FFastGeoHLOD::GetCustomHLODGuid() const
{
	return GetHLODGuid(EHLODGuidType::Custom);
}

const FGuid& FFastGeoHLOD::GetHLODGuid(EHLODGuidType InGuidType) const
{
	if (HLODGuidType == InGuidType)
	{
		return HLODGuid;
	}
	else
	{
		static const FGuid InvalidGuid;
		return InvalidGuid;
	}
}

void FFastGeoHLOD::OnRegister()
{
	GetLevel()->GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->RegisterHLODObject(this);
}

void FFastGeoHLOD::OnUnregister()
{
	GetLevel()->GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->UnregisterHLODObject(this);
}

#if WITH_EDITOR
void FFastGeoHLOD::SetSourceCellGuid(const FGuid& InSourceCellGuid)
{
	SourceCellGuid = InSourceCellGuid;
}

void FFastGeoHLOD::SetRequireWarmup(bool bInRequireWarmup)
{
	bRequireWarmup = bInRequireWarmup;
}

void FFastGeoHLOD::SetStandaloneHLODGuid(const FGuid& InStandaloneHLODGuid)
{
	SetHLODGuid(InStandaloneHLODGuid, EHLODGuidType::Standalone);
}

void FFastGeoHLOD::SetCustomHLODGuid(const FGuid& InCustomHLODGuid)
{
	SetHLODGuid(InCustomHLODGuid, EHLODGuidType::Custom);
}

void FFastGeoHLOD::SetHLODGuid(const FGuid& InGuid, EHLODGuidType InGuidType)
{
	HLODGuid = InGuid;
	HLODGuidType = InGuidType;
}

FFastGeoComponent& FFastGeoHLOD::AddComponent(FFastGeoElementType InComponentType)
{
	FFastGeoComponent& NewComponent = Super::AddComponent(InComponentType);

	if (FFastGeoPrimitiveComponent* PrimitiveComponent = NewComponent.CastTo<FFastGeoPrimitiveComponent>())
	{
		// Always disable collisions on HLODs
		PrimitiveComponent->SetCollisionEnabled(false);
	}

	return NewComponent;
}

void FFastGeoHLOD::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// When cooking, get rid of collision data
	if (ObjectSaveContext.IsCooking())
	{
		ForEachComponent<FFastGeoStaticMeshComponentBase>([this](const FFastGeoStaticMeshComponentBase& StaticMeshComponent)
		{
			if (UStaticMesh* StaticMesh = StaticMeshComponent.GetStaticMesh())
			{
				// If the HLOD process did create this static mesh
				if (StaticMesh->GetPackage() == GetOwnerContainer()->GetPackage())
				{
					if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
					{
						// To ensure a deterministic cook, save the current GUID and restore it below
						FGuid PreviousBodySetupGuid = BodySetup->BodySetupGuid;
						BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
						BodySetup->bNeverNeedsCookedCollisionData = true;
						BodySetup->bHasCookedCollisionData = false;
						BodySetup->InvalidatePhysicsData();
						BodySetup->BodySetupGuid = PreviousBodySetupGuid;
					}
				}
			}
		});
	}
}
#endif