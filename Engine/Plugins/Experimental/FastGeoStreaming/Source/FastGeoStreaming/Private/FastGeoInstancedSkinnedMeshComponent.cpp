// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoInstancedSkinnedMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "ContentStreaming.h"
#include "Engine/MaterialOverlayHelper.h"
#include "FastGeoHLOD.h"
#include "InstanceData/InstanceUpdateChangeSet.h"
#include "InstancedSkinnedMeshComponentHelper.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"
#include "SkeletalRenderPublic.h"
#include "SkinnedMeshComponentHelper.h"

const FFastGeoElementType FFastGeoInstancedSkinnedMeshComponent::Type(&FFastGeoSkinnedMeshComponentBase::Type);

FFastGeoInstancedSkinnedMeshComponent::FFastGeoInstancedSkinnedMeshComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
	check(!InstanceDataManager.IsValid());
	InstanceDataManager = MakeShared<FInstanceDataManager>(nullptr);
}

void FFastGeoInstancedSkinnedMeshComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << InstanceData;
	Ar << NumCustomDataFloats;
	Ar << InstanceCustomData;

	// Serialize persistent data from FInstancedSkinnedMeshSceneProxyDesc
	Ar << SceneProxyDesc.AnimationMinScreenSize;
	Ar << SceneProxyDesc.InstanceMinDrawDistance;
	Ar << SceneProxyDesc.InstanceStartCullDistance;
	Ar << SceneProxyDesc.InstanceEndCullDistance;
	Ar << SceneProxyDesc.TransformProvider;
	Ar << SceneProxyDesc.bUseGpuLodSelection;
	Ar << SceneProxyDesc.bForceAnimateSockets;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bAllowAlwaysVisible);
}
	
#if WITH_EDITOR
void FFastGeoInstancedSkinnedMeshComponent::InitializeSceneProxyDescFromComponent(UActorComponent* Component)
{
	UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = CastChecked<UInstancedSkinnedMeshComponent>(Component);
	SceneProxyDesc.InitializeFromInstancedSkinnedMeshComponent(InstancedSkinnedMeshComponent);
}

void FFastGeoInstancedSkinnedMeshComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = CastChecked<UInstancedSkinnedMeshComponent>(Component);

	InstanceData = InstancedSkinnedMeshComponent->GetInstanceData();
	NumCustomDataFloats = InstancedSkinnedMeshComponent->GetNumCustomDataFloats();
	InstanceCustomData = InstancedSkinnedMeshComponent->GetInstanceCustomData();

	LocalBounds = FInstancedSkinnedMeshComponentHelper::CalcBounds(*this, FTransform::Identity);
	WorldBounds = FInstancedSkinnedMeshComponentHelper::CalcBounds(*this, WorldTransform);

	// ISKMC with no instances should never be transformed to FastGeo
	check(!InstanceData.IsEmpty());
}
	
void FFastGeoInstancedSkinnedMeshComponent::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();
}
#endif

void FFastGeoInstancedSkinnedMeshComponent::InitializeSceneProxyDescDynamicProperties()
{
	Super::InitializeSceneProxyDescDynamicProperties();

	SceneProxyDesc.InstanceDataSceneProxy = InstanceDataManager->GetOrCreateProxy();

	InstanceDataManager->FlushChanges(FInstancedSkinnedMeshComponentHelper::GetComponentDesc<FFastGeoInstancedSkinnedMeshComponent, /*bSupportHitProxies=*/false>(*this, GetScene()->GetShaderPlatform()));
}

void FFastGeoInstancedSkinnedMeshComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	Super::ApplyWorldTransform(InTransform);
	WorldBounds = FInstancedSkinnedMeshComponentHelper::CalcBounds(*this, InTransform);
}

FPrimitiveSceneProxy* FFastGeoInstancedSkinnedMeshComponent::AllocateSceneProxy()
{
	return FInstancedSkinnedMeshComponentHelper::CreateSceneProxy(*this, SceneProxyDesc);
}

FSkeletalMeshObject* FFastGeoInstancedSkinnedMeshComponent::CreateMeshObject()
{
	return FInstancedSkinnedMeshComponentHelper::CreateMeshObject(*this, SceneProxyDesc);
}

void FFastGeoInstancedSkinnedMeshComponent::UpdateSkinning()
{
	// FInstancedSkeletalMeshObject::Update consumes only LODIndex, but DynamicData must still be
	// seeded or UpdateSceneExtensionHeader asserts on check(DynamicData) for the GPU-skin path.
	FSkinnedMeshSceneProxyDynamicData DynamicData;
	DynamicData.bRenderStateRecreating = true;
	RunMeshObjectUpdate(DynamicData);
}
