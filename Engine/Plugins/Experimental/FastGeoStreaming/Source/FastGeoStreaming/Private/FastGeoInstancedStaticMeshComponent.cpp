// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoInstancedStaticMeshComponent.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoSurrogateBodyInstanceIndex.h"
#include "FastGeoContainer.h"
#include "FastGeoHLOD.h"
#include "FastGeoLog.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Engine/InstancedStaticMesh.h"
#include "InstancedStaticMeshComponentHelper.h"
#include "InstanceData/InstanceDataHelpers.h"
#include "InstancedStaticMesh/ISMInstanceDataSceneProxy.h"
#include "NaniteVertexFactory.h"
#include "NaniteSceneProxy.h"
#include "PhysicsEngine/BodySetup.h"
#include "Math/DoubleFloat.h"
#include "SceneInterface.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

const FFastGeoElementType FFastGeoInstancedStaticMeshComponent::Type(&FFastGeoStaticMeshComponentBase::Type);

FFastGeoInstancedStaticMeshComponent::FFastGeoInstancedStaticMeshComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

void FFastGeoInstancedStaticMeshComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FFastGeoInstancedStaticMeshComponent
	FArchive_Serialize_BitfieldBool(Ar, bUseHighPrecisionPerInstanceSMData);
	if (bUseHighPrecisionPerInstanceSMData)
	{
		HighPrecisionPerInstanceSMData.BulkSerialize(Ar);
	}
	else
	{
		LowPrecisionPerInstanceSMData.BulkSerialize(Ar);
	}
	Ar << LastInstanceBodyIndex;
	Ar << InstancingRandomSeed;
	PerInstanceSMCustomData.BulkSerialize(Ar);
	Ar << AdditionalRandomSeeds;
	Ar << NavigationBounds;

	// Serialize persistent data from FInstancedStaticMeshSceneProxyDesc
	Ar << SceneProxyDesc.InstanceLODDistanceScale;
	Ar << SceneProxyDesc.InstanceMinDrawDistance;
	Ar << SceneProxyDesc.InstanceStartCullDistance;
	Ar << SceneProxyDesc.InstanceEndCullDistance;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bUseGpuLodSelection);

	// Precomputed spatial hashes and pre-generated random IDs for instance hierarchy optimization
	SpatialHashes.BulkSerialize(Ar);
	PerInstanceRandomIDs.BulkSerialize(Ar);
}

#if WITH_EDITOR
void FFastGeoInstancedStaticMeshComponent::InitializeSceneProxyDescFromComponent(UActorComponent* Component)
{
	UInstancedStaticMeshComponent* InstancedStaticMeshComponent = CastChecked<UInstancedStaticMeshComponent>(Component);
	SceneProxyDesc.InitializeFromInstancedStaticMeshComponent(InstancedStaticMeshComponent);
}

void FFastGeoInstancedStaticMeshComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UInstancedStaticMeshComponent* ISMComponent = CastChecked<UInstancedStaticMeshComponent>(Component);
	SceneProxyDesc.bCollisionEnabled = !!SceneProxyDesc.bCollisionEnabled && !ISMComponent->bDisableCollision;
	AdditionalRandomSeeds = ISMComponent->AdditionalRandomSeeds;
	InitializePerInstanceSMData(ISMComponent->PerInstanceSMData);
	PerInstanceSMCustomData = ISMComponent->PerInstanceSMCustomData;
	InstancingRandomSeed = ISMComponent->InstancingRandomSeed;

	// ISMC with no instances should never be transformed to FastGeo
	check(GetInstanceCount() > 0);

	LocalBounds = CalculateBounds(EBoundsType::LocalBounds);
	WorldBounds = CalculateBounds(EBoundsType::WorldBounds);
	NavigationBounds = CalculateBounds(EBoundsType::NavigationBounds).GetBox();

	// Compute spatial hash order and physically reorder instance data for optimal RT performance.
	// This sorts instances by spatial locality so the render thread can use precomputed spatial
	// hashes for batch insertion into the instance hierarchy instead of per-instance processing.
	{
		const int32 InstanceCount = GetInstanceCount();
		const FRenderBounds MeshBounds = GetStaticMesh()->GetBounds();
		const FMatrix ComponentTransformMatrix = WorldTransform.ToMatrixWithScale();

		// Compute PrimitiveToRelativeWorld and world space offset (same as FInstanceDataManager does)
		const FVector3f PrimitiveWorldSpacePositionHigh = FDFVector3{ ComponentTransformMatrix.GetOrigin() }.High;
		const FRenderTransform PrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(PrimitiveWorldSpacePositionHigh, ComponentTransformMatrix).M;
		const FVector PrimitiveWorldSpaceOffset(PrimitiveWorldSpacePositionHigh);

		// Build spatial hash data and get reorder table (empty if already in optimal order)
		FInstanceDataHelpers::FSpatialHashResult SpatialHashResult = FInstanceDataHelpers::BuildSpatialHashData(InstanceCount, [&](int32 InstanceIndex) -> FSphere
		{
			FMatrix InstanceTransform;
			if (bUseHighPrecisionPerInstanceSMData)
			{
				InstanceTransform = HighPrecisionPerInstanceSMData[InstanceIndex].Transform;
			}
			else
			{
				InstanceTransform = FMatrix(LowPrecisionPerInstanceSMData[InstanceIndex].ToMatrixWithScale());
			}
			const FSphere3f LocalSphere(MeshBounds.GetCenter(), MeshBounds.GetExtent().Size());
			const FRenderTransform InstanceLocalToWorld = FRenderTransform(InstanceTransform) * PrimitiveToRelativeWorld;
			FSphere Result(LocalSphere.TransformBy(InstanceLocalToWorld.ToMatrix44f()));
			Result.Center += PrimitiveWorldSpaceOffset;
			return Result;
		});

		SpatialHashes = MoveTemp(SpatialHashResult.SpatialHashes);
		const TArray<int32>& ReorderTable = SpatialHashResult.ReorderTable;
		const bool bNeedsReorder = ReorderTable.Num() > 0;

		// Physically reorder instance data arrays
		if (bNeedsReorder)
		{
			if (bUseHighPrecisionPerInstanceSMData)
			{
				TArray<FInstancedStaticMeshInstanceData> Reordered;
				Reordered.SetNumUninitialized(InstanceCount);
				for (int32 Index = 0; Index < InstanceCount; ++Index)
				{
					Reordered[Index] = HighPrecisionPerInstanceSMData[ReorderTable[Index]];
				}
				HighPrecisionPerInstanceSMData = MoveTemp(Reordered);
			}
			else
			{
				TArray<FTransform3f> Reordered;
				Reordered.SetNumUninitialized(InstanceCount);
				for (int32 Index = 0; Index < InstanceCount; ++Index)
				{
					Reordered[Index] = LowPrecisionPerInstanceSMData[ReorderTable[Index]];
				}
				LowPrecisionPerInstanceSMData = MoveTemp(Reordered);
			}

			// Reorder custom data (strided by NumCustomDataFloats)
			const int32 NumCustomDataFloats = PerInstanceSMCustomData.Num() / FMath::Max(InstanceCount, 1);
			if (NumCustomDataFloats > 0)
			{
				const SIZE_T CustomDataStride = NumCustomDataFloats * sizeof(float);
				TArray<float> ReorderedCustomData;
				ReorderedCustomData.SetNumUninitialized(PerInstanceSMCustomData.Num());
				for (int32 Index = 0; Index < InstanceCount; ++Index)
				{
					const int32 OldIndex = ReorderTable[Index];
					FMemory::Memcpy(&ReorderedCustomData[Index * NumCustomDataFloats], &PerInstanceSMCustomData[OldIndex * NumCustomDataFloats], CustomDataStride);
				}
				PerInstanceSMCustomData = MoveTemp(ReorderedCustomData);
			}
		}

		// Pre-generate random IDs in the reordered order
		{
			TArray<float> OriginalRandomIDs = FInstanceDataHelpers::GenerateInstanceRandomIDs(InstanceCount, InstancingRandomSeed, AdditionalRandomSeeds);
			if (bNeedsReorder)
			{
				PerInstanceRandomIDs.SetNumUninitialized(InstanceCount);
				for (int32 Index = 0; Index < InstanceCount; ++Index)
				{
					PerInstanceRandomIDs[Index] = OriginalRandomIDs[ReorderTable[Index]];
				}
			}
			else
			{
				PerInstanceRandomIDs = MoveTemp(OriginalRandomIDs);
			}
		}
	}
}

void FFastGeoInstancedStaticMeshComponent::InitializePerInstanceSMData(const TArray<FInstancedStaticMeshInstanceData>& InPerInstanceSMData)
{
	const UFastGeoWorldPartitionRuntimeCellTransformer* Transformer = UFastGeoWorldPartitionRuntimeCellTransformer::GetCurrentTransformer();
	const UFastGeoTransformerSettings& Settings = Transformer ? Transformer->GetSettings() : *GetDefault<UFastGeoTransformerSettings>();
	const EFastGeoInstanceStorageMode InstanceStorageMode = Settings.InstanceStorageMode;
	const float PositionThreshold = Settings.InstanceCompressionSettings.PositionEpsilon;
	const float RotationThreshold = Settings.InstanceCompressionSettings.RotationEpsilon;
	const float ScaleThreshold = Settings.InstanceCompressionSettings.ScaleEpsilon;
	const bool bAutomatic = (InstanceStorageMode == EFastGeoInstanceStorageMode::Auto);
	bUseHighPrecisionPerInstanceSMData = (InstanceStorageMode == EFastGeoInstanceStorageMode::Full);
	HighPrecisionPerInstanceSMData.Empty();
	LowPrecisionPerInstanceSMData.Empty();

	if (!bUseHighPrecisionPerInstanceSMData)
	{
		LowPrecisionPerInstanceSMData.Reserve(InPerInstanceSMData.Num());
		for (const FInstancedStaticMeshInstanceData& InstanceData : InPerInstanceSMData)
		{
			const FTransform OrigTransform(InstanceData.Transform);
			const FTransform3f CompressedTransform(OrigTransform);

			if (bAutomatic)
			{
				const FVector OrigLocation = OrigTransform.GetLocation();
				const FQuat OrigRotation = OrigTransform.GetRotation();
				const FVector OrigScale = OrigTransform.GetScale3D();

				const FVector NewPos = (FVector)CompressedTransform.GetTranslation();
				const FQuat NewRot = (FQuat)CompressedTransform.GetRotation();
				const FVector NewScale = (FVector)CompressedTransform.GetScale3D();

				const bool bPosOk = OrigLocation.Equals(NewPos, PositionThreshold);
				const bool bRotOk = OrigRotation.Equals(NewRot, RotationThreshold);
				const bool bScaleOk = OrigScale.Equals(NewScale, ScaleThreshold);
				if (!bPosOk || !bRotOk || !bScaleOk)
				{
					bUseHighPrecisionPerInstanceSMData = true;
					break;
				}
			}

			LowPrecisionPerInstanceSMData.Emplace(CompressedTransform);
		}
	}

	if (bUseHighPrecisionPerInstanceSMData)
	{
		LowPrecisionPerInstanceSMData.Empty();
		HighPrecisionPerInstanceSMData = InPerInstanceSMData;
	}
}

void FFastGeoInstancedStaticMeshComponent::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	SceneProxyDesc.InstanceDataSceneProxy = nullptr;
	SceneProxyDesc.bHasSelectedInstances = false;
}
#endif

void FFastGeoInstancedStaticMeshComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	Super::ApplyWorldTransform(InTransform);

	bAppliedWorldTransform = true;
	WorldBounds = CalculateBounds(EBoundsType::WorldBounds);
	NavigationBounds = CalculateBounds(EBoundsType::NavigationBounds).GetBox();
}

void FFastGeoInstancedStaticMeshComponent::ForEachInstanceMatrix(TFunctionRef<void(const FMatrix&)> Func) const
{
	if (bUseHighPrecisionPerInstanceSMData)
	{
		for (const FInstancedStaticMeshInstanceData& InstanceData : HighPrecisionPerInstanceSMData)
		{
			Func(InstanceData.Transform);
		}
	}
	else
	{
		for (const FTransform3f& Transform: LowPrecisionPerInstanceSMData)
		{
			Func(FMatrix(Transform.ToMatrixWithScale()));
		}
	}
}

int32 FFastGeoInstancedStaticMeshComponent::GetInstanceCount() const
{
	return bUseHighPrecisionPerInstanceSMData ? HighPrecisionPerInstanceSMData.Num() : LowPrecisionPerInstanceSMData.Num();
}

void FFastGeoInstancedStaticMeshComponent::BuildInstanceData()
{
	FInstanceSceneDataBuffers InstanceSceneDataBuffers{};
	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView View = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	// PrimitiveLocalToWorld
	InstanceSceneDataBuffers.SetPrimitiveLocalToWorld(GetRenderMatrix(), AccessTag);

	// InstanceLocalBounds
	const FPrimitiveMaterialPropertyDescriptor PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(GetScene()->GetShaderPlatform());
	const float LocalAbsMaxDisplacement = FMath::Max(-PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y) + PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;
	const FVector3f PadExtent = FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(View.PrimitiveToRelativeWorld, LocalAbsMaxDisplacement);
	FRenderBounds InstanceLocalBounds = GetStaticMesh()->GetBounds();
	InstanceLocalBounds.Min -= PadExtent;
	InstanceLocalBounds.Max += PadExtent;
	check(!View.Flags.bHasPerInstanceLocalBounds);
	View.InstanceLocalBounds.Add(InstanceLocalBounds);

	// LocalToPrimitiveRelativeWorld
	const int32 InstanceCount = GetInstanceCount();
	View.InstanceToPrimitiveRelative.Reserve(InstanceCount);
	if (bUseHighPrecisionPerInstanceSMData)
	{
		for (const FInstancedStaticMeshInstanceData& InstanceData : HighPrecisionPerInstanceSMData)
		{
			FRenderTransform LocalToPrimitiveRelativeWorld = FRenderTransform(InstanceData.Transform) * View.PrimitiveToRelativeWorld;
			LocalToPrimitiveRelativeWorld.Orthogonalize();
			View.InstanceToPrimitiveRelative.Add(LocalToPrimitiveRelativeWorld);
		}
	}
	else
	{
		for (const FTransform3f& Transform : LowPrecisionPerInstanceSMData)
		{
			FRenderTransform LocalToPrimitiveRelativeWorld = FRenderTransform(Transform.ToMatrixWithScale()) * View.PrimitiveToRelativeWorld;
			LocalToPrimitiveRelativeWorld.Orthogonalize();
			View.InstanceToPrimitiveRelative.Add(LocalToPrimitiveRelativeWorld);
		}
	}

	// InstanceCustomData
	const int32 NumCustomDataFloats = (PerInstanceSMCustomData.Num() && InstanceCount) ? PerInstanceSMCustomData.Num() / InstanceCount : 0;
	View.InstanceCustomData = PerInstanceSMCustomData;
	View.NumCustomDataFloats = NumCustomDataFloats;
	View.Flags.bHasPerInstanceCustomData = PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData && NumCustomDataFloats != 0;
	if (!View.Flags.bHasPerInstanceCustomData)
	{
		View.NumCustomDataFloats = 0;
		View.InstanceCustomData.Reset();
	}

	// InstanceRandomIDs (pre-generated in spatial hash order from cook)
	View.Flags.bHasPerInstanceRandom = PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom && (InstanceCount > 0);
	if (View.Flags.bHasPerInstanceRandom)
	{
		View.InstanceRandomIDs = PerInstanceRandomIDs;
	}

	// Precomputed spatial hashes (no reorder table - data already reordered at cook time).
	// If a world transform was applied at stream-in, spatial hash cell locations are invalid
	// since they encode cook-time world positions. Reordered data still provides cache locality.
	if (!SpatialHashes.IsEmpty() && !bAppliedWorldTransform)
	{
		InstanceSceneDataBuffers.SetImmutable(FInstanceSceneDataImmutable(SpatialHashes), AccessTag);
	}

	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	InstanceSceneDataBuffers.ValidateData();

	SceneProxyDesc.InstanceDataSceneProxy = MakeShared<FInstanceDataSceneProxy, ESPMode::ThreadSafe>(MoveTemp(InstanceSceneDataBuffers));
}

FPrimitiveSceneProxy* FFastGeoInstancedStaticMeshComponent::CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	check(GetWorld());
	check(SceneProxyDesc.Scene);
	check(GetInstanceCount() > 0);

	BuildInstanceData();

	if (bCreateNanite)
	{
		PrimitiveSceneData.SceneProxy = ::new Nanite::FSceneProxy(NaniteMaterials, SceneProxyDesc);
	}
	else
	{
		PrimitiveSceneData.SceneProxy = ::new FInstancedStaticMeshSceneProxy(SceneProxyDesc, SceneProxyDesc.FeatureLevel);
	}

	// No need to keep this around, as keeping a reference would force the various instance scene data buffers to only
	// be released at GC time. Releasing the ref will allow the data to be freed outside of the game thread.
	SceneProxyDesc.InstanceDataSceneProxy.Reset();

	return PrimitiveSceneData.SceneProxy;
}

#if WITH_EDITOR
void FFastGeoInstancedStaticMeshComponent::OnAsyncCreatePhysicsStateBegin_GameThread()
{
	// GT-side capture of WalkableSlopeOverride from the (potentially designer-mutable) BodySetup.
	// The worker step (CreateAllInstanceBodies) skips its own read in editor (#if !WITH_EDITOR).
	// In cooked, BodySetup is immutable post-PostLoad and the worker read is uncontested.
	if (UBodySetup* BodySetup = GetBodySetup())
	{
		if (!BodyInstance.GetOverrideWalkableSlopeOnInstance())
		{
			BodyInstance.SetWalkableSlopeOverride(BodySetup->WalkableSlopeOverride, false);
		}
	}

	Super::OnAsyncCreatePhysicsStateBegin_GameThread();
}
#endif

void FFastGeoInstancedStaticMeshComponent::OnAsyncCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoInstancedStaticMeshComponent::OnAsyncCreatePhysicsState);

	check(InstanceBodies.Num() == 0);

	FPhysScene* PhysScene = GetPhysicsScene();
	if (!PhysScene)
	{
		return;
	}

	// Create all the bodies.
	CreateAllInstanceBodies();

	FFastGeoComponent::OnAsyncCreatePhysicsState();
}

void FFastGeoInstancedStaticMeshComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	for (FBodyInstance* Body : InstanceBodies)
	{
		if (Body)
		{
			Body->ClearCachedPhysicsCreationInputs();
		}
	}

	Super::OnAsyncCreatePhysicsStateEnd_GameThread();
}

void FFastGeoInstancedStaticMeshComponent::OnAsyncDestroyPhysicsStateBegin_GameThread()
{
	FFastGeoComponent::OnAsyncDestroyPhysicsStateBegin_GameThread();

	// UnRegisterForCollisionEvents on game thread before going async.
	// TermBody skips UnRegister when called from a worker thread (see FBodyInstance::TermBody contract).
	if (BodyInstance.bNotifyRigidBodyCollision)
	{
		if (UFastGeoSurrogateComponent* SurrogateComponent = GetSurrogateComponent())
		{
			if (FPhysScene* PhysScene = GetPhysicsScene())
			{
				PhysScene->UnRegisterForCollisionEvents(SurrogateComponent);
			}
		}
	}

	// Move InstanceBodies in AsyncDestroyPhysicsStatePayload
	check(AsyncDestroyPhysicsStatePayload.IsEmpty());
	AsyncDestroyPhysicsStatePayload = MoveTemp(InstanceBodies);
}

void FFastGeoInstancedStaticMeshComponent::OnAsyncDestroyPhysicsStateEnd_GameThread()
{
	FFastGeoComponent::OnAsyncDestroyPhysicsStateEnd_GameThread();

	// Reset BodyInstanceOwner
	BodyInstanceOwner.Uninitialize();
}

void FFastGeoInstancedStaticMeshComponent::OnAsyncDestroyPhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoInstancedStaticMeshComponent::OnAsyncDestroyPhysicsState);

	FFastGeoComponent::OnAsyncDestroyPhysicsState();

	// Remove all user defined entities
	TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects(AsyncDestroyPhysicsStatePayload);
	FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects)->SetUserDefinedEntity(PhysicsObjects, nullptr);

	check(InstanceBodies.IsEmpty());
	for (FBodyInstance*& Instance : AsyncDestroyPhysicsStatePayload)
	{
		if (Instance)
		{
			Instance->TermBody();
			delete Instance;
		}
	}
	AsyncDestroyPhysicsStatePayload.Empty();
}

Chaos::FPhysicsObject* FFastGeoInstancedStaticMeshComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!InstanceBodies.IsValidIndex(Id) || !InstanceBodies[Id] || !InstanceBodies[Id]->GetPhysicsActor())
	{
		return nullptr;
	}
	return InstanceBodies[Id]->GetPhysicsActor()->GetPhysicsObject();
}

TArray<Chaos::FPhysicsObject*> FFastGeoInstancedStaticMeshComponent::GetAllPhysicsObjects() const
{
	return GetAllPhysicsObjects(InstanceBodies);
}

TArray<Chaos::FPhysicsObject*> FFastGeoInstancedStaticMeshComponent::GetAllPhysicsObjects(const TArray<FBodyInstance*>& InInstanceBodies)
{
	TArray<Chaos::FPhysicsObject*> Objects;
	Objects.Reserve(InInstanceBodies.Num());
	for (FBodyInstance* InstancedBody : InInstanceBodies)
	{
		if (Chaos::FPhysicsObject* PhysicsObject = InstancedBody && InstancedBody->GetPhysicsActor() ? InstancedBody->GetPhysicsActor()->GetPhysicsObject() : nullptr)
		{
			Objects.Add(PhysicsObject);
		}
	}
	return Objects;
}

FBodyInstance* FFastGeoInstancedStaticMeshComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	if (Index != INDEX_NONE)
	{
		// Handle the case where GetBodyInstance is called directly on the FastGeo component with an already transformed Index
		if (UFastGeoSurrogateComponent* SurrogateComponent = GetSurrogateComponent())
		{
			if (FFastGeoSurrogateBodyInstanceIndex::IsEncoded(Index))
			{
				return SurrogateComponent->GetBodyInstance(BoneName, bGetWelded, Index);
			}
		}
		else
		{
			check(BodyInstance.InstanceBodyIndex == INDEX_NONE);
			if (InstanceBodies.IsValidIndex(Index))
			{
				return const_cast<FBodyInstance*>(InstanceBodies[Index]);
			}
		}
	}
	return const_cast<FBodyInstance*>(&BodyInstance);
}

void FFastGeoInstancedStaticMeshComponent::CreateAllInstanceBodies()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoInstancedStaticMeshComponent::CreateAllInstanceBodies);
	
	const int32 NumBodies = GetInstanceCount();
	check(InstanceBodies.Num() == 0);
	check(SceneProxyDesc.Mobility != EComponentMobility::Movable);

	if (UBodySetup* BodySetup = GetBodySetup())
	{
		FPhysScene* PhysScene = GetPhysicsScene();

#if !WITH_EDITOR
		// Cooked: BodySetup is immutable post-PostLoad, safe to read on worker. In editor this is
		// hoisted to OnAsyncCreatePhysicsStateBegin_GameThread to avoid the designer-mid-edit race
		// during PIE.
		if (!BodyInstance.GetOverrideWalkableSlopeOnInstance())
		{
			BodyInstance.SetWalkableSlopeOverride(BodySetup->WalkableSlopeOverride, false);
		}
#endif

		InstanceBodies.SetNumUninitialized(NumBodies);

		// Sanitized array does not contain any nulls
		TArray<FBodyInstance*> InstanceBodiesSanitized;
		InstanceBodiesSanitized.Reserve(NumBodies);

		UFastGeoSurrogateComponent* SurrogateComponent = GetSurrogateComponent();
		const bool bUseSurrogateComponent = !!SurrogateComponent;
		check(!bUseSurrogateComponent || FFastGeoSurrogateBodyInstanceIndex::IsEncoded(BodyInstance.InstanceBodyIndex));
		const int32 BaseOffset = bUseSurrogateComponent ? BodyInstance.InstanceBodyIndex : 0;

		TArray<FTransform> Transforms;
		Transforms.Reserve(NumBodies);
		int32 InstanceBodyIndex = 0;
		ForEachInstanceMatrix([this, BaseOffset, &InstanceBodyIndex, &Transforms, &InstanceBodiesSanitized](const FMatrix& InstanceTransform)
		{
			const FTransform InstanceTM = FTransform(InstanceTransform) * WorldTransform;
			if (InstanceTM.GetScale3D().IsNearlyZero())
			{
				InstanceBodies[InstanceBodyIndex] = nullptr;
			}
			else
			{
				FBodyInstance* Instance = new FBodyInstance;
				InstanceBodiesSanitized.Add(Instance);
				InstanceBodies[InstanceBodyIndex] = Instance;
				// Struct-copy propagates the prototype's PhysicsCreationInputs cache for worker-side InitStaticBodies.
				Instance->CopyBodyInstancePropertiesFrom(&BodyInstance);
				Instance->InstanceBodyIndex = BaseOffset + InstanceBodyIndex;
				Instance->bAutoWeld = false;
				Instance->bSimulatePhysics = false;
				Transforms.Add(InstanceTM);
			}
			++InstanceBodyIndex;
		});

		check(!bUseSurrogateComponent || InstanceBodiesSanitized.IsEmpty() || InstanceBodiesSanitized.Last()->InstanceBodyIndex <= LastInstanceBodyIndex);

		if (InstanceBodiesSanitized.Num() > 0)
		{
			// Initialize BodyInstanceOwner
			BodyInstanceOwner.Initialize(this);

			// Initialize body instances
			FBodyInstance::InitStaticBodies(MoveTemp(InstanceBodiesSanitized), MoveTemp(Transforms), BodySetup, SurrogateComponent, GetPhysicsScene(), &BodyInstanceOwner);

			// Assign BodyInstanceOwner
			TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects(InstanceBodies);
			FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects)->SetUserDefinedEntity(PhysicsObjects, &BodyInstanceOwner);
		}
	}
	else
	{
		// In case we get into some bad state where the BodySetup is invalid but bPhysicsStateCreated is true,
		// issue a warning and add nullptrs to InstanceBodies.
		UE_LOGF(LogFastGeoStreaming, Warning, "Instance Static Mesh Component unable to create InstanceBodies!");
		InstanceBodies.AddZeroed(NumBodies);
	}
}

FBoxSphereBounds FFastGeoInstancedStaticMeshComponent::CalculateBounds(EBoundsType BoundsType)
{
	if (GetStaticMesh() && GetInstanceCount() > 0)
	{
		const bool bWorldSpace = (BoundsType != EBoundsType::LocalBounds);
		const FBox InstanceBounds = (BoundsType == EBoundsType::NavigationBounds) ? FInstancedStaticMeshComponentHelper::GetInstanceNavigationBounds(*this) : GetStaticMesh()->GetBounds().GetBox();
		const FMatrix ComponentTransformMatrix = WorldTransform.ToMatrixWithScale();
		if (InstanceBounds.IsValid)
		{
			FBoxSphereBounds::Builder BoundsBuilder;
			ForEachInstanceMatrix([bWorldSpace, &InstanceBounds, &ComponentTransformMatrix, &BoundsBuilder](const FMatrix& InstanceTransform)
			{
				if (bWorldSpace)
				{
					BoundsBuilder += InstanceBounds.TransformBy(InstanceTransform * ComponentTransformMatrix);
				}
				else
				{
					BoundsBuilder += InstanceBounds.TransformBy(InstanceTransform);
				}
			});
			return BoundsBuilder;
		}
	}
	return FBoxSphereBounds(FVector::ZeroVector, FVector::ZeroVector, 0.f);
};

void FFastGeoInstancedStaticMeshComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	FInstancedStaticMeshComponentHelper::GetNavigationData(*this, Data, FNavDataPerInstanceTransformDelegate::CreateWeakLambda(GetOwnerContainer(), [this](const FBox& AreaBox, TArray<FTransform>& InstanceData)
	{
		FInstancedStaticMeshComponentHelper::GetNavigationPerInstanceTransforms(*this, AreaBox, InstanceData);
	}));
}

FBox FFastGeoInstancedStaticMeshComponent::GetNavigationBounds() const
{
	return NavigationBounds;
}

bool FFastGeoInstancedStaticMeshComponent::IsNavigationRelevant() const
{
	return GetInstanceCount() > 0 && Super::IsNavigationRelevant();
}

bool FFastGeoInstancedStaticMeshComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	return FInstancedStaticMeshComponentHelper::DoCustomNavigableGeometryExport(*this, GeomExport, FNavDataPerInstanceTransformDelegate::CreateWeakLambda(GetOwnerContainer(), [this](const FBox& AreaBox, TArray<FTransform>& InstanceData)
	{
		FInstancedStaticMeshComponentHelper::GetNavigationPerInstanceTransforms(*this, AreaBox, InstanceData);
	}));
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FFastGeoInstancedStaticMeshComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	return FInstancedStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}
#endif

void FFastGeoInstancedStaticMeshComponent::GetBodyInstances(TArray<FBodyInstance*>& OutBodyInstances)
{
	OutBodyInstances.Append(InstanceBodies);
}

#if WITH_EDITOR
void FFastGeoInstancedStaticMeshComponent::ReserveSurrogateInstanceBodyIndices(FFastGeoSurrogateBodyInstanceIndex& InOutNextAvailableInstanceBodyIndex)
{
	const int32 InstanceCount = GetInstanceCount();
	if (InstanceCount > 0)
	{
		const int32 RangeOffset = InstanceCount - 1;
		Super::ReserveSurrogateInstanceBodyIndices(InOutNextAvailableInstanceBodyIndex);
		check(FFastGeoSurrogateBodyInstanceIndex::IsEncoded(BodyInstance.InstanceBodyIndex));
		LastInstanceBodyIndex = (FFastGeoSurrogateBodyInstanceIndex::FromEncoded(BodyInstance.InstanceBodyIndex) + RangeOffset).GetEncoded();
		InOutNextAvailableInstanceBodyIndex += RangeOffset;
	}
	else
	{
		check(BodyInstance.InstanceBodyIndex == INDEX_NONE);
		check(LastInstanceBodyIndex == INDEX_NONE);
	}
}
#endif