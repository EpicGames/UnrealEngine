// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoLocalLightComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoPSOWeakPtrTraits.h"
#include "Components/LocalLightComponent.h"
#include "PointLightSceneProxy.h"
#include "PointLightSceneProxyDesc.h"
#include "LightComponentId.h"
#include "LightSceneDesc.h"
#include "RenderUtils.h"
#include "SceneInterface.h"

const FFastGeoElementType FFastGeoLocalLightComponent::Type(&FFastGeoComponent::Type);

FFastGeoLocalLightComponent::FFastGeoLocalLightComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
	, bAffectsWorld(true)
	, bSourceComponentIsVisible(true)
	, bIsVisible(true)
	, SceneProxy(nullptr)
{
}

FFastGeoLocalLightComponent::FFastGeoLocalLightComponent(const FFastGeoLocalLightComponent& Other)
	: Super(Other)
	, bAffectsWorld(Other.bAffectsWorld)
	, bSourceComponentIsVisible(Other.bSourceComponentIsVisible)
	, bIsVisible(true)
	, SceneProxy(nullptr)
{
}

#if WITH_EDITOR
void FFastGeoLocalLightComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	ULocalLightComponent* LocalLightComponent = CastChecked<ULocalLightComponent>(Component);
	bAffectsWorld = LocalLightComponent->bAffectsWorld;
	bSourceComponentIsVisible = LocalLightComponent->IsVisible() && LocalLightComponent->GetOwner() && !LocalLightComponent->GetOwner()->IsHidden();
}
#endif

void FFastGeoLocalLightComponent::ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const
{
	const FLocalLightSceneProxyDesc& SceneProxyDesc = GetLocalLightSceneProxyDesc();
	if (SceneProxyDesc.LightFunctionMaterial)
	{
		Func(SceneProxyDesc.LightFunctionMaterial, false);
	}
}

void FFastGeoLocalLightComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data
	Ar << bAffectsWorld;
	Ar << bSourceComponentIsVisible;

	// Serialize persistent data from FLocalLightSceneProxyDesc
	FLocalLightSceneProxyDesc& SceneProxyDesc = GetLocalLightSceneProxyDesc();
	SceneProxyDesc.Serialize(Ar);

	if (Ar.IsPersistent() && Ar.IsLoading())
	{
		// Initialize LightComponentId as it's not serialized
		SceneProxyDesc.LightComponentId = FLightComponentId::GetNextId();
	}
}

bool FFastGeoLocalLightComponent::ShouldComponentAddToRenderScene() const
{
	return Super::ShouldComponentAddToRenderScene() && bAffectsWorld && bSourceComponentIsVisible && bIsVisible;
}

void FFastGeoLocalLightComponent::CreateRenderState(FRegisterComponentContext* Context)
{
	InitializeSceneProxyDescDynamicProperties();
	if (!ShouldComponentAddToRenderScene())
	{
		ProxyState = EProxyCreationState::None;
		return;
	}

	FWriteScopeLock WriteLock(*Lock.Get());
	check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
	ProxyState = EProxyCreationState::Creating;
	bRenderStateDirty = false;

	check(!SceneProxy);
	ESceneProxyCreationError Error;
	SceneProxy = CreateSceneProxy(Error);
	if (SceneProxy)
	{
		check(bIsVisible);
		FLightSceneDesc LightSceneDesc{ .Transform = WorldTransform, .SceneProxy = SceneProxy};
		FSceneInterface* Scene = GetScene();
		check(Scene);

		Scene->BatchAddLights(MakeArrayView(&LightSceneDesc, 1));

		check(LightSceneDesc.SceneProxy);
		ProxyState = EProxyCreationState::Created;
	}
	else if (Error == ESceneProxyCreationError::WaitingPSOs)
	{
		ProxyState = EProxyCreationState::Delayed;
	}
	else
	{
		ProxyState = EProxyCreationState::None;
	}
}

FLightSceneProxy* FFastGeoLocalLightComponent::CreateSceneProxy(ESceneProxyCreationError& OutError)
{
	check(ShouldComponentAddToRenderScene());

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Skip PSO delay during blocking wait -- GT cannot pump tasks so PSO callbacks won't fire.
	const bool bForceCreate = GetOwnerContainer()->GetWorldSubsystem()->IsWaitingForCompletion();
	if (PSOPrecacheComponentData.IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate && !bForceCreate)
	{
		OutError = ESceneProxyCreationError::WaitingPSOs;
		return nullptr;
	}
#endif

	OutError = ESceneProxyCreationError::None;
	FSceneInterface* Scene = GetScene();
	check(Scene);
	FLocalLightSceneProxyDesc& SceneProxyDesc = GetLocalLightSceneProxyDesc();
	SceneProxyDesc.SceneInterface = Scene;
	return CreateTypedSceneProxy();
}

void FFastGeoLocalLightComponent::DestroyRenderState(FFastGeoDestroyRenderStateContext* Context)
{
	FWriteScopeLock WriteLock(*Lock.Get());
	if (SceneProxy)
	{
		check(ProxyState == EProxyCreationState::Created);
		FLightSceneDesc LightSceneDesc{ .Transform = WorldTransform, .SceneProxy = SceneProxy };
		FSceneInterface* Scene = GetScene();
		check(Scene);
		Scene->BatchRemoveLights(MakeArrayView(&LightSceneDesc, 1));
		SceneProxy = nullptr;
		ProxyState = EProxyCreationState::None;
		bRenderStateDirty = false;
	}
	else
	{
		check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
		ProxyState = EProxyCreationState::None;
	}
}

void FFastGeoLocalLightComponent::InitializeSceneProxyDescDynamicProperties()
{
	UpdateVisibilityInternal();
}

void FFastGeoLocalLightComponent::UpdateVisibility()
{
	if (UpdateVisibilityInternal())
	{
		MarkRenderStateDirty(true);
	}
}

bool FFastGeoLocalLightComponent::UpdateVisibilityInternal()
{
	bool bOldIsVisible = bIsVisible;
#if !UE_BUILD_SHIPPING
	bIsVisible = UFastGeoWorldSubsystem::IsFastGeoVisible();
#else
	bIsVisible = true;
#endif
	return bIsVisible != bOldIsVisible;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
bool FFastGeoLocalLightComponent::ShouldPrecachePSOs() const
{
	if (!Super::ShouldPrecachePSOs())
	{
		return false;
	}

	const FLocalLightSceneProxyDesc& SceneProxyDesc = GetLocalLightSceneProxyDesc();
	return SceneProxyDesc.LightFunctionMaterial && !SceneProxyDesc.LightFunctionMaterial->HasAnyFlags(RF_NeedPostLoad);
}

void FFastGeoLocalLightComponent::PrecachePSOs_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoLocalLightComponent::PrecachePSOs_Concurrent);
	if (!ShouldPrecachePSOs())
	{
		return;
	}

	FPSOPrecacheParams PSOPrecacheParams;
	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
	VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));

	// Immediately create at high priority and thus doesn't need boosting anymore
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	FLocalLightSceneProxyDesc& SceneProxyDesc = GetLocalLightSceneProxyDesc();
	FGraphEventArray PSOPrecacheCompileEvents = SceneProxyDesc.LightFunctionMaterial->PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);

	auto OnPrecacheFinishedGT = [WeakComponent = FWeakFastGeoComponent(this)]()
	{
		if (FFastGeoLocalLightComponent* Component = WeakComponent.Get<FFastGeoLocalLightComponent>())
		{
			if (UFastGeoContainer* Container = Component->GetOwnerContainer())
			{
				if (Container->GetWorld())
				{
					Container->OnComponentPSOPrecacheCompleted(Component);
				}
			}
		}
	};

	PSOPrecacheComponentData.SetMaterialPSOPrecacheData(this, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents, OnPrecacheFinishedGT);
}
#endif