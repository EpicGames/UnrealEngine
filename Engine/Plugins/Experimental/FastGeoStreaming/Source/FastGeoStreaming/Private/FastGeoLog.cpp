// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoLog.h"

DEFINE_LOG_CATEGORY(LogFastGeoStreaming);