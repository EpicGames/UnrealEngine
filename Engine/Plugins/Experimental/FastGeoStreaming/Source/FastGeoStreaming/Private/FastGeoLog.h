// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

FASTGEOSTREAMING_API DECLARE_LOG_CATEGORY_EXTERN(LogFastGeoStreaming, Log, All);