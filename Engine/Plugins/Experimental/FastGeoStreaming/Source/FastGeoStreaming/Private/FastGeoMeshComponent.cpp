// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoMeshComponent.h"

const FFastGeoElementType FFastGeoMeshComponent::Type(&FFastGeoPrimitiveComponent::Type);

FFastGeoMeshComponent::FFastGeoMeshComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}	

void FFastGeoMeshComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FFastGeoMeshComponent
	Ar << OverrideMaterials;
}

#if WITH_EDITOR
void FFastGeoMeshComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UMeshComponent* MeshComponent = CastChecked<UMeshComponent>(Component);
	OverrideMaterials = MeshComponent->OverrideMaterials;
}
#endif

void FFastGeoMeshComponent::GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	FMeshComponentHelper::GetMaterialSlotsOverlayMaterial(*this, OutMaterialSlotOverlayMaterials);
}

FMaterialRelevance FFastGeoMeshComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	return FMeshComponentHelper::GetMaterialRelevance(*this, InShaderPlatform);
}
