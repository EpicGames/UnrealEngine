// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoPhysicsBodyInstanceOwner.h"
#include "FastGeoPrimitiveComponent.h"
#include "FastGeoContainer.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

const FName FFastGeoPhysicsBodyInstanceOwner::NAME_FastGeoPhysicsBodyInstanceOwner = TEXT("FastGeoPhysicsBodyInstanceOwner");

FFastGeoPhysicsBodyInstanceOwner::FFastGeoPhysicsBodyInstanceOwner()
	: FChaosUserDefinedEntity(FFastGeoPhysicsBodyInstanceOwner::NAME_FastGeoPhysicsBodyInstanceOwner)
	, OwnerComponent(nullptr)
{
}

void FFastGeoPhysicsBodyInstanceOwner::Uninitialize()
{
	Initialize(nullptr);
}

void FFastGeoPhysicsBodyInstanceOwner::Initialize(FFastGeoPrimitiveComponent* InOwnerComponent)
{
	OwnerComponent = InOwnerComponent;
	UFastGeoContainer* NewContainer = OwnerComponent ? OwnerComponent->GetOwnerContainer() : nullptr;
	check(!OwnerComponent || NewContainer);
	check(OwnerContainer.IsExplicitlyNull() || !NewContainer || OwnerContainer == NewContainer);
	OwnerContainer = NewContainer;
}

TWeakObjectPtr<UObject> FFastGeoPhysicsBodyInstanceOwner::GetOwnerObject()
{
	return OwnerContainer;
}

IPhysicsBodyInstanceOwner* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwner(FChaosUserDefinedEntity* InUserDefinedEntity)
{
	if (InUserDefinedEntity && InUserDefinedEntity->GetEntityTypeName() == NAME_FastGeoPhysicsBodyInstanceOwner)
	{
		FFastGeoPhysicsBodyInstanceOwner* FastGeoPhysicsBodyInstanceOwner = static_cast<FFastGeoPhysicsBodyInstanceOwner*>(InUserDefinedEntity);
		check(FastGeoPhysicsBodyInstanceOwner->GetOwnerObject().IsValid());
		return FastGeoPhysicsBodyInstanceOwner;
	}
	return nullptr;
}

const FFastGeoPhysicsBodyInstanceOwner* FFastGeoPhysicsBodyInstanceOwner::FromPhysicsObject(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject)
	{
		FLockedReadPhysicsObjectExternalInterface PhysicsObjectInterface = FPhysicsObjectExternalInterface::LockRead(PhysicsObject);
		FChaosUserDefinedEntity* UserDefinedEntity = PhysicsObjectInterface->GetUserDefinedEntity(PhysicsObject);
		if (UserDefinedEntity && UserDefinedEntity->GetEntityTypeName() == NAME_FastGeoPhysicsBodyInstanceOwner)
		{
			check(UserDefinedEntity->GetOwnerObject().IsValid());
			return static_cast<const FFastGeoPhysicsBodyInstanceOwner*>(UserDefinedEntity);
		}
	}
	return nullptr;
}

const FFastGeoPhysicsBodyInstanceOwner* FFastGeoPhysicsBodyInstanceOwner::FromHitResult(const FHitResult& HitResult)
{
	if (HitResult.PhysicsObject && HitResult.PhysicsObjectOwner.IsValid())
	{
		return FromPhysicsObject(HitResult.PhysicsObject);
	}
	return nullptr;
}

const FFastGeoPhysicsBodyInstanceOwner* FFastGeoPhysicsBodyInstanceOwner::FromOverlapResult(const FOverlapResult& OverlapResult)
{
	if (OverlapResult.PhysicsObject && OverlapResult.PhysicsObjectOwner.IsValid())
	{
		return FromPhysicsObject(OverlapResult.PhysicsObject);
	}
	return nullptr;
}

bool FFastGeoPhysicsBodyInstanceOwner::IsStaticPhysics() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->IsStaticPhysics();
}

bool FFastGeoPhysicsBodyInstanceOwner::IsMultiBodyOverlap() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->IsMultiBodyOverlap();
}

UObject* FFastGeoPhysicsBodyInstanceOwner::GetSourceObject() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetSourceObject();
}

FTransform FFastGeoPhysicsBodyInstanceOwner::GetPhysicsOwnerTransform() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetComponentTransform();
}

ECollisionChannel FFastGeoPhysicsBodyInstanceOwner::GetCollisionObjectType() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetCollisionObjectType();
}

ECollisionEnabled::Type FFastGeoPhysicsBodyInstanceOwner::GetCollisionEnabled() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetCollisionEnabled();
}

ECollisionResponse FFastGeoPhysicsBodyInstanceOwner::GetCollisionResponseToChannel(ECollisionChannel Channel) const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetCollisionResponseToChannel(Channel);
}

Chaos::FPhysicsObject* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetPhysicsObjectById(Id);
}

TArray<Chaos::FPhysicsObject*> FFastGeoPhysicsBodyInstanceOwner::GetAllPhysicsObjects() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetAllPhysicsObjects();
}

FBodyInstance* FFastGeoPhysicsBodyInstanceOwner::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetBodyInstance(BoneName, bGetWelded, Index);
}

UPhysicalMaterial* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsMaterialOverride() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetPhysicsMaterialOverride();
}

UMaterialInterface* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsMaterialBase() const
{
	check(OwnerContainer.IsValid());
	if (FFastGeoMeshComponent* MeshComponent = OwnerComponent->CastTo<FFastGeoMeshComponent>())
	{
		return MeshComponent->GetMaterial(0);
	}
	return nullptr;
}

int32 FFastGeoPhysicsBodyInstanceOwner::GetNumMaterials() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetNumMaterials();
}

UMaterialInterface* FFastGeoPhysicsBodyInstanceOwner::GetMaterial(int32 Index) const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetMaterial(Index);
}

UBodySetup* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsBodySetup() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetBodySetup();
}

UObject* FFastGeoPhysicsBodyInstanceOwner::GetSourceObjectOwner() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetOwner();
}

FTransform FFastGeoPhysicsBodyInstanceOwner::GetPhysicsOwnerSocketTransform(FName InSocketName) const
{
	return FTransform::Identity;
}

bool FFastGeoPhysicsBodyInstanceOwner::IsPhysicsOwnerMovable() const
{
	// FastGeo doesn't support movables
	return false;
}

bool FFastGeoPhysicsBodyInstanceOwner::IsPhysicsOwnerSimulatingPhysics() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->BodyInstance.IsInstanceSimulatingPhysics();
}

FVector FFastGeoPhysicsBodyInstanceOwner::GetPhysicsOwnerVelocity() const
{
	// FastGeo doesn't support movables so return empty vector
	return FVector::ZeroVector;
}

UObject* FFastGeoPhysicsBodyInstanceOwner::GetPhysicsOwnerAttachmentRoot() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->GetSourceObject();
}

bool FFastGeoPhysicsBodyInstanceOwner::IsPhysicsObjectWorldGeometry() const
{
	return true;
}

bool FFastGeoPhysicsBodyInstanceOwner::DoesSocketExistOnPhysicsOwner(FName InSocketName) const
{
	return false;
}

const FWalkableSlopeOverride& FFastGeoPhysicsBodyInstanceOwner::GetWalkableSlopeOverride() const
{
	check(OwnerContainer.IsValid());
	return OwnerComponent->BodyInstance.GetWalkableSlopeOverride();
}
