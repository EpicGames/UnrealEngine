// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoPointLightComponent.h"
#include "Components/PointLightComponent.h"
#include "PointLightSceneProxy.h"
#include "PointLightSceneProxyDesc.h"
#include "RenderUtils.h"
#include "SceneInterface.h"

const FFastGeoElementType FFastGeoPointLightComponentBase::Type(&FFastGeoLocalLightComponent::Type);

FFastGeoPointLightComponentBase::FFastGeoPointLightComponentBase(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

bool FFastGeoPointLightComponentBase::ShouldComponentAddToRenderScene() const
{
	return Super::ShouldComponentAddToRenderScene() && IsPointLightSupported();
}

bool FFastGeoPointLightComponentBase::IsPointLightSupported() const
{
	const FPointLightSceneProxyDesc& SceneProxyDesc = GetPointLightSceneProxyDesc();
	if (GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1 && SceneProxyDesc.bMovable)
	{
		if (!IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform))
		{
			// if project does not support dynamic point lights on mobile do not add them to the renderer 
			return MobileForwardEnableLocalLights(GMaxRHIShaderPlatform);
		}
	}
	return true;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
bool FFastGeoPointLightComponentBase::ShouldPrecachePSOs() const
{
	return Super::ShouldPrecachePSOs() && IsPointLightSupported();
}
#endif

const FFastGeoElementType FFastGeoPointLightComponent::Type(&FFastGeoPointLightComponentBase::Type);

FFastGeoPointLightComponent::FFastGeoPointLightComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

#if WITH_EDITOR
void FFastGeoPointLightComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UPointLightComponent* PointLightComponent = CastChecked<UPointLightComponent>(Component);
	SceneProxyDesc = UPointLightComponent::BuildSceneProxyDesc(*PointLightComponent);
	SceneProxyDesc.LightComponent = nullptr;
}
#endif

FLightSceneProxy* FFastGeoPointLightComponent::CreateTypedSceneProxy()
{
	check(SceneProxyDesc.SceneInterface == GetScene());
	return new FPointLightSceneProxy(SceneProxyDesc);
}