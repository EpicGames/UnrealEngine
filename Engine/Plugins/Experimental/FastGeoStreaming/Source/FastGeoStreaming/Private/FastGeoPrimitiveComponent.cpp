// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoPrimitiveComponent.h"
#include "FastGeoContainer.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoInstancedStaticMeshComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoSurrogateBodyInstanceIndex.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoSurrogateActor.h"
#include "FastGeoDestroyRenderStateContext.h"
#include "FastGeoWeakElement.h"
#include "FastGeoLog.h"
#include "FastGeoPSOWeakPtrTraits.h"
#include "AI/NavigationModifier.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "PrimitiveSceneDesc.h"
#include "SceneInterface.h"
#include "WorldPartition/WorldPartition.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PrimitiveComponentHelper.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "UnrealEngine.h"

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#endif

const FFastGeoElementType FFastGeoPrimitiveComponent::Type(&FFastGeoComponent::Type);

FFastGeoPrimitiveComponent::FFastGeoPrimitiveComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
	, LocalBounds(ForceInit)
	, WorldBounds(ForceInit)
{
}

FFastGeoPrimitiveComponent::FFastGeoPrimitiveComponent(const FFastGeoPrimitiveComponent& Other)
	: Super(Other)
	, LocalBounds(Other.LocalBounds)
	, WorldBounds(Other.WorldBounds)
	, bIsVisible(Other.bIsVisible)
	, bStaticWhenNotMoveable(Other.bStaticWhenNotMoveable)
	, bFillCollisionUnderneathForNavmesh(Other.bFillCollisionUnderneathForNavmesh)
	, bRasterizeAsFilledConvexVolume(Other.bRasterizeAsFilledConvexVolume)
	, bCanEverAffectNavigation(Other.bCanEverAffectNavigation)
	, bMultiBodyOverlap(Other.bMultiBodyOverlap)
	, SurrogateComponentDescriptorIndex(Other.SurrogateComponentDescriptorIndex)
	, CustomPrimitiveData(Other.CustomPrimitiveData)
	, bHasCustomNavigableGeometry(Other.bHasCustomNavigableGeometry)
	, BodyInstance(Other.BodyInstance)
	, RuntimeVirtualTextures(Other.RuntimeVirtualTextures)
	, BodyInstanceOwner()
	, PrimitiveSceneData()
	, AsyncTermBodyPayload()
{
}

void FFastGeoPrimitiveComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FFastGeoPrimitiveComponent
	Ar << LocalBounds;
	Ar << WorldBounds;
	FArchive_Serialize_BitfieldBool(Ar, bIsVisible);
	FArchive_Serialize_BitfieldBool(Ar, bStaticWhenNotMoveable);
	FArchive_Serialize_BitfieldBool(Ar, bFillCollisionUnderneathForNavmesh);
	FArchive_Serialize_BitfieldBool(Ar, bRasterizeAsFilledConvexVolume);
	FArchive_Serialize_BitfieldBool(Ar, bCanEverAffectNavigation);
	FArchive_Serialize_BitfieldBool(Ar, bMultiBodyOverlap);
	Ar << SurrogateComponentDescriptorIndex;
	Ar << CustomPrimitiveData.Data;
	Ar << bHasCustomNavigableGeometry;
	Ar << RuntimeVirtualTextures;

	// Serialize persistent data from FPrimitiveSceneProxyDesc
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.CastShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bReceivesDecals);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bOnlyOwnerSee);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bOwnerNoSee);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bUseViewOwnerDepthPriorityGroup);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bVisibleInReflectionCaptures);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bVisibleInRealTimeSkyCaptures);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bVisibleInRayTracing);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bVisibleInReflections);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bRenderInDepthPass);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bRenderInMainPass);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bTreatAsBackgroundForOcclusion);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastDynamicShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastStaticShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bEmissiveLightSource);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bAffectDynamicIndirectLighting);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bAffectIndirectLightingWhileHidden);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bAffectDistanceFieldLighting);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastVolumetricTranslucentShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastContactShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastHiddenShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastShadowAsTwoSided);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bSelfShadowOnly);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastInsetShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastCinematicShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastFarShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bLightAttachmentsAsGroup);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bSingleSampleShadowFromStationaryLights);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bUseAsOccluder);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bHasPerInstanceHitProxies);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bReceiveMobileCSMShadows);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bRenderCustomDepth);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bVisibleInSceneCaptureOnly);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bHiddenInSceneCapture);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bForceMipStreaming);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bRayTracingFarField);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bHoldout);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bIsFirstPerson);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bIsFirstPersonWorldSpaceRepresentation);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCollisionEnabled);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bIsHidden);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bSupportsWorldPositionOffsetVelocity);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bIsInstancedStaticMesh);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bHasStaticLighting);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bHasValidSettingsForStaticLighting);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bIsPrecomputedLightingValid);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bShadowIndirectOnly);
	Ar << SceneProxyDesc.Mobility;
	Ar << SceneProxyDesc.TranslucencySortPriority;
	Ar << SceneProxyDesc.TranslucencySortDistanceOffset;
	Ar << SceneProxyDesc.LightmapType;
	Ar << SceneProxyDesc.ViewOwnerDepthPriorityGroup;
	Ar << SceneProxyDesc.CustomDepthStencilValue;
	Ar << SceneProxyDesc.CustomDepthStencilWriteMask;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.LightingChannels.bChannel0);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.LightingChannels.bChannel1);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.LightingChannels.bChannel2);
	Ar << SceneProxyDesc.RayTracingGroupCullingPriority;
	Ar << SceneProxyDesc.IndirectLightingCacheQuality;
	Ar << SceneProxyDesc.ShadowCacheInvalidationBehavior;
	Ar << SceneProxyDesc.DepthPriorityGroup;
	Ar << SceneProxyDesc.VirtualTextureLodBias;
	Ar << SceneProxyDesc.VirtualTextureCullMips;
	Ar << SceneProxyDesc.VirtualTextureMinCoverage;
	Ar << SceneProxyDesc.VisibilityId;
	Ar << SceneProxyDesc.CachedMaxDrawDistance;
	Ar << SceneProxyDesc.MinDrawDistance;
	Ar << SceneProxyDesc.BoundsScale;
	Ar << SceneProxyDesc.RayTracingGroupId;
	Ar << SceneProxyDesc.VirtualTextureRenderPassType;
	Ar << SceneProxyDesc.VirtualTextureMainPassMaxDrawDistance;

	// Zero-init + reconstruct BodyInstance to match UObject memzero behavior
	// for non-UPROPERTY fields that SerializeItem won't restore.
	if (Ar.IsLoading())
	{
		BodyInstance.~FBodyInstance();
		FMemory::Memzero(&BodyInstance, sizeof(FBodyInstance));
		new (&BodyInstance) FBodyInstance();
	}

	// Skip BodyInstance serialization when collision is disabled (e.g. HLODs)
	if (SceneProxyDesc.bCollisionEnabled)
	{
		FBodyInstance::StaticStruct()->SerializeItem(Ar, &BodyInstance, nullptr);
		Ar << BodyInstance.InstanceBodyIndex;
	}

	if (Ar.IsLoading() && !UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents())
	{
		BodyInstance.InstanceBodyIndex = INDEX_NONE;
	}
}

#if WITH_EDITOR

void FFastGeoPrimitiveComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	// Initialize properties not handled by InitializeFromPrimitiveComponent
	UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Component);
	bIsVisible = PrimitiveComponent->IsVisible();
	bStaticWhenNotMoveable = PrimitiveComponent->GetStaticWhenNotMoveable();
	bFillCollisionUnderneathForNavmesh = PrimitiveComponent->bFillCollisionUnderneathForNavmesh;
	bRasterizeAsFilledConvexVolume = PrimitiveComponent->bRasterizeAsFilledConvexVolume;
	bCanEverAffectNavigation = PrimitiveComponent->CanEverAffectNavigation();
	bMultiBodyOverlap = PrimitiveComponent->bMultiBodyOverlap;
	SurrogateComponentDescriptorIndex = INDEX_NONE;
	CustomPrimitiveData = PrimitiveComponent->GetCustomPrimitiveData();
	bHasCustomNavigableGeometry = PrimitiveComponent->bHasCustomNavigableGeometry;
	BodyInstance.CopyBodyInstancePropertiesFrom(&PrimitiveComponent->BodyInstance);
	RuntimeVirtualTextures = PrimitiveComponent->GetRuntimeVirtualTextures();

	// Initialize SceneProxyDesc from component
	InitializeSceneProxyDescFromComponent(Component);

	// Reset some values that are not used in FastGeo
	ResetSceneProxyDescUnsupportedProperties();
}

void FFastGeoPrimitiveComponent::ResetSceneProxyDescUnsupportedProperties()
{
	// Unsupported properties
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.bLevelInstanceEditingState = false;
	SceneProxyDesc.bSelectable = false;
	SceneProxyDesc.bUseEditorCompositing = false;
	SceneProxyDesc.bIsBeingMovedByEditor = false;
	SceneProxyDesc.bSelected = false;
	SceneProxyDesc.bIndividuallySelected = false;
	SceneProxyDesc.bShouldRenderSelected = false;
	SceneProxyDesc.bWantsEditorEffects = false;
	SceneProxyDesc.bIsHiddenEd = false;
	SceneProxyDesc.bIsOwnerEditorOnly = false;
	SceneProxyDesc.bIsOwnedByFoliage = false;
	SceneProxyDesc.HiddenEditorViews = 0;
	SceneProxyDesc.OverlayColor = FColor(EForceInit::ForceInitToZero);
	SceneProxyDesc.Component = nullptr;

	// Properties that will be initialized by InitializeSceneProxyDescDynamicProperties
	SceneProxyDesc.ComponentId = FPrimitiveComponentId();
	SceneProxyDesc.StatId = TStatId();
	SceneProxyDesc.Owner = nullptr;
	SceneProxyDesc.World = nullptr;
	SceneProxyDesc.CustomPrimitiveData = nullptr;
	SceneProxyDesc.Scene = nullptr;
	SceneProxyDesc.PrimitiveComponentInterface = nullptr;
	SceneProxyDesc.FeatureLevel = ERHIFeatureLevel::Num;
	SceneProxyDesc.RuntimeVirtualTextures = TArrayView<URuntimeVirtualTexture*>();
	SceneProxyDesc.bIsVisible = false;
	SceneProxyDesc.bUsePSOPrecacheFallbackMaterial = false;
#if MESH_DRAW_COMMAND_STATS
	SceneProxyDesc.MeshDrawCommandStatsCategory = NAME_None;
#endif
}
#endif

bool FFastGeoPrimitiveComponent::IsDrawnInGame() const
{
	// Drawn in game must consider both the component bIsVisible flag AND the bIsHidden flag (which actually originates from the actor bHiddenInGame property)
	// This logic mimics what is done to initialize FPrimitiveSceneProxy::DrawInGame
	return GetSceneProxyDesc().bIsVisible && !GetSceneProxyDesc().bIsHidden;
}

EComponentMobility::Type FFastGeoPrimitiveComponent::GetMobility() const
{
	return GetSceneProxyDesc().Mobility;
}

void FFastGeoPrimitiveComponent::UpdateVisibility()
{
	// Update SceneProxyDesc.bIsVisible as it's dependant on component and component cluster visibility
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.bIsVisible = bIsVisible && GetOwnerComponentCluster()->IsVisible();
#if !UE_BUILD_SHIPPING
	SceneProxyDesc.bIsVisible = SceneProxyDesc.bIsVisible && UFastGeoWorldSubsystem::IsFastGeoVisible();
#endif
	SceneProxyDesc.bIsVisibleEditor = SceneProxyDesc.bIsVisible;
}

void FFastGeoPrimitiveComponent::InitializeSceneProxyDescDynamicProperties()
{
	check(GetWorld());
	check(GetScene());

#if WITH_EDITOR
	ResetSceneProxyDescUnsupportedProperties();
#endif

	// Initialize non-serialized properties
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.ComponentId = GetPrimitiveSceneId();
	UObject const* AdditionalStatObjectPtr = AdditionalStatObject();
	SceneProxyDesc.StatId = AdditionalStatObjectPtr ? AdditionalStatObjectPtr->GetStatID(true) : TStatId();
	SceneProxyDesc.Owner = GetOwnerContainer();
#if !WITH_STATE_STREAM
	SceneProxyDesc.World = GetWorld();
#endif
	SceneProxyDesc.CustomPrimitiveData = &CustomPrimitiveData;
	SceneProxyDesc.Scene = GetScene();
#if WITH_EDITOR
	SceneProxyDesc.PrimitiveComponentInterface = this;
#endif
	SceneProxyDesc.FeatureLevel = SceneProxyDesc.Scene->GetFeatureLevel();
	TArray<URuntimeVirtualTexture*> const& VirtualTextures = GetRuntimeVirtualTextures();
	SceneProxyDesc.RuntimeVirtualTextures = MakeArrayView(const_cast<URuntimeVirtualTexture**>(VirtualTextures.GetData()), VirtualTextures.Num());

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	SceneProxyDesc.SetPSOPrecacheComponentData(PSOPrecacheComponentData);
#endif // !WITH_EDITOR && UE_WITH_PSO_PRECACHING

#if MESH_DRAW_COMMAND_STATS
	static FName NAME_FastGeoPrimitiveComponent(TEXT("FastGeoPrimitiveComponent"));
	SceneProxyDesc.MeshDrawCommandStatsCategory = NAME_FastGeoPrimitiveComponent;
#endif
	check(SceneProxyDesc.bIsInstancedStaticMesh == (this->IsA<FFastGeoInstancedStaticMeshComponent>() || this->IsA<FFastGeoProceduralISMComponent>()));
	UpdateVisibility();
}

FPrimitiveSceneDesc FFastGeoPrimitiveComponent::BuildSceneDesc()
{
	check(GetSceneProxy());

	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	FPrimitiveSceneDesc SceneDesc;
	SceneDesc.SceneProxy = GetSceneProxy();
	SceneDesc.ProxyDesc = &SceneProxyDesc;
	SceneDesc.PrimitiveSceneData = &PrimitiveSceneData;
	SceneDesc.RenderMatrix = GetRenderMatrix();
	SceneDesc.AttachmentRootPosition = GetTransform().GetTranslation();
	SceneDesc.LocalBounds = LocalBounds;
	SceneDesc.Bounds = GetBounds();
	SceneDesc.Mobility = SceneProxyDesc.Mobility;

	return SceneDesc;
}

void FFastGeoPrimitiveComponent::CreateRenderState(FRegisterComponentContext* Context)
{
	if (!ShouldComponentAddToRenderScene())
	{
		ProxyState = EProxyCreationState::None;
		return;
	}

	FWriteScopeLock WriteLock(*Lock.Get());
	// Delayed is valid: PSO-deferred components re-enter CreateRenderState once PSOs are ready.
	check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
	ProxyState = EProxyCreationState::Creating;
	bRenderStateDirty = false;

#if WITH_EDITOR
	NotifyRenderStateChanged();
#endif
	
	FSceneInterface* Scene = GetScene();
	check(Scene);

	ESceneProxyCreationError Error;
	if (FPrimitiveSceneProxy* SceneProxy = CreateSceneProxy(&Error))
	{
		SceneProxy->SetPrimitiveColor(GetDebugColor());
		check(GetSceneProxy());
		FPrimitiveSceneDesc Desc = BuildSceneDesc();
		Scene->AddPrimitive(&Desc);

		ProxyState = EProxyCreationState::Created;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		bCreatedWithPSOFallbackMaterial = GetSceneProxyDesc().bUsePSOPrecacheFallbackMaterial;
#endif
	}
	else if (Error == ESceneProxyCreationError::WaitingPSOs)
	{
		ProxyState = EProxyCreationState::Delayed;
	}
	else
	{
		ProxyState = EProxyCreationState::None;
	}
}

FString FFastGeoPrimitiveComponent::GetName() const
{
	return GetNameSafe(GetOwnerContainer());
}

FString FFastGeoPrimitiveComponent::GetFullName() const
{
	return GetFullNameSafe(GetOwnerContainer());
}

void FFastGeoPrimitiveComponent::DestroyRenderState(FFastGeoDestroyRenderStateContext* Context)
{
	FWriteScopeLock WriteLock(*Lock.Get());
	if (GetSceneProxy())
	{
		FSceneInterface* Scene = GetScene();
		check(Scene);

		check(ProxyState == EProxyCreationState::Created);

		FFastGeoDestroyRenderStateContext::DestroyProxy(Context, GetSceneProxy(), Scene);

		PrimitiveSceneData.SceneProxy = nullptr;

		ProxyState = EProxyCreationState::None;

		bRenderStateDirty = false;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		bCreatedWithPSOFallbackMaterial = false;
#endif

#if WITH_EDITOR
		NotifyRenderStateChanged();
#endif
	}
	else
	{
		check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
		ProxyState = EProxyCreationState::None;
	}
}

#if WITH_EDITOR
void FFastGeoPrimitiveComponent::NotifyRenderStateChanged()
{
	FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(this);
}
#endif

void FFastGeoPrimitiveComponent::OnAsyncCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoPrimitiveComponent::OnAsyncCreatePhysicsState);

	Super::OnAsyncCreatePhysicsState();

	// if we have a scene, we don't want to disable all physics and we have no bodyinstance already
	if (!BodyInstance.IsValidBodyInstance())
	{
		if (UBodySetup* BodySetup = GetBodySetup())
		{
			// Create new BodyInstance at given location.
			FTransform BodyTransform = WorldTransform;

			// Here we make sure we don't have zero scale. This still results in a body being made and placed in
			// world (very small) but is consistent with a body scaled to zero.
			const FVector BodyScale = BodyTransform.GetScale3D();
			if (BodyScale.IsNearlyZero())
			{
				BodyTransform.SetScale3D(FVector(UE_KINDA_SMALL_NUMBER));
			}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if ((BodyInstance.GetCollisionEnabled() != ECollisionEnabled::NoCollision) && (FMath::IsNearlyZero(BodyScale.X) || FMath::IsNearlyZero(BodyScale.Y) || FMath::IsNearlyZero(BodyScale.Z)))
			{
				UE_LOGF(LogFastGeoStreaming, Warning, "Scale for FastGeoPrimitiveComponent has a component set to zero, which will result in a bad body instance. Scale:%ls", *BodyScale.ToString());

				// User warning has been output - fix up the scale to be valid for physics
				BodyTransform.SetScale3D(FVector(
					FMath::IsNearlyZero(BodyScale.X) ? UE_KINDA_SMALL_NUMBER : BodyScale.X,
					FMath::IsNearlyZero(BodyScale.Y) ? UE_KINDA_SMALL_NUMBER : BodyScale.Y,
					FMath::IsNearlyZero(BodyScale.Z) ? UE_KINDA_SMALL_NUMBER : BodyScale.Z
				));
			}
#endif

			// Initialize BodyInstanceOwner
			BodyInstanceOwner.Initialize(this);

			UFastGeoSurrogateComponent* SurrogateComponent = GetSurrogateComponent();
			// Make sure that the BodyInstance.InstanceBodyIndex was properly set by the FastGeo runtime cell transformer
			check(!SurrogateComponent || FFastGeoSurrogateBodyInstanceIndex::IsEncoded(BodyInstance.InstanceBodyIndex));

			// Initialize the body instance
			BodyInstance.InitBody(BodySetup, BodyTransform, SurrogateComponent, GetPhysicsScene(), FInitBodySpawnParams(IsStaticPhysics(), /*bPhysicsTypeDeterminesSimulation*/false), &BodyInstanceOwner);

			// Assign BodyInstanceOwner
			if (Chaos::FSingleParticlePhysicsProxy* ProxyHandle = BodyInstance.GetPhysicsActor())
			{
				if (Chaos::FPhysicsObject* PhysicsObject = BodyInstance.IsValidBodyInstance() ? BodyInstance.GetPhysicsActor()->GetPhysicsObject() : nullptr)
				{
					TArrayView<Chaos::FPhysicsObject*> PhysicsObjects(&PhysicsObject, 1);
					FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects)->SetUserDefinedEntity(PhysicsObjects, &BodyInstanceOwner);
				}
			}
		}
	}
}

void FFastGeoPrimitiveComponent::OnAsyncDestroyPhysicsStateBegin_GameThread()
{
	check(!AsyncTermBodyPayload.IsSet());
	AsyncTermBodyPayload = BodyInstance.StartAsyncTermBody_GameThread();
	check(!BodyInstance.IsValidBodyInstance());

	Super::OnAsyncDestroyPhysicsStateBegin_GameThread();
}

void FFastGeoPrimitiveComponent::OnAsyncDestroyPhysicsStateEnd_GameThread()
{
	Super::OnAsyncDestroyPhysicsStateEnd_GameThread();

	// Reset BodyInstanceOwner
	BodyInstanceOwner.Uninitialize();
}

void FFastGeoPrimitiveComponent::OnAsyncDestroyPhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoPrimitiveComponent::OnAsyncDestroyPhysicsState);

	// We tell the BodyInstance to shut down the physics-engine data.
	if (ensure(AsyncTermBodyPayload.IsSet()))
	{
		// Remove all user defined entities
		if (Chaos::FPhysicsObject* PhysicsObject = AsyncTermBodyPayload->GetPhysicsActor() ? AsyncTermBodyPayload->GetPhysicsActor()->GetPhysicsObject() : nullptr)
		{
			TArrayView<Chaos::FPhysicsObject*> PhysicsObjects(&PhysicsObject, 1);
			FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects)->SetUserDefinedEntity(PhysicsObjects, nullptr);
		}

		FBodyInstance::AsyncTermBody(AsyncTermBodyPayload.GetValue());
		AsyncTermBodyPayload.Reset();
	}

	Super::OnAsyncDestroyPhysicsState();
}

bool FFastGeoPrimitiveComponent::IsRenderStateDirty() const
{
	return Super::IsRenderStateDirty();
}

bool FFastGeoPrimitiveComponent::IsRenderStateCreated() const
{
	return Super::IsRenderStateCreated();
}

bool FFastGeoPrimitiveComponent::ShouldComponentAddToRenderScene() const
{
	if (!Super::ShouldComponentAddToRenderScene())
	{
		return false;
	}

	const FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	const bool bShouldCreateRenderState = (bIsVisible && !SceneProxyDesc.bIsHidden) || SceneProxyDesc.bCastHiddenShadow || SceneProxyDesc.bAffectIndirectLightingWhileHidden || SceneProxyDesc.bRayTracingFarField;
	return bShouldCreateRenderState;
}

bool FFastGeoPrimitiveComponent::ShouldCreateRenderState() const
{
	return ShouldComponentAddToRenderScene();
}

bool FFastGeoPrimitiveComponent::IsRegistered() const
{
	// Keep behavior where registering was also considered registered
	if (UFastGeoContainer* Container = GetOwnerContainer(); ensure(Container))
	{
		return Container->IsRegistered() || Container->IsRegistering();
	}
	return false;
}

bool FFastGeoPrimitiveComponent::IsUnreachable() const
{
	const UObject* OwnerPtr = GetOwnerContainer();
	return !OwnerPtr || OwnerPtr->IsUnreachable();
}

bool FFastGeoPrimitiveComponent::IsStaticMobility() const
{
	return true;
}

bool FFastGeoPrimitiveComponent::IsMipStreamingForced() const
{
	return false;
}

UWorld* FFastGeoPrimitiveComponent::GetWorld() const
{
	return Super::GetWorld();
}

FSceneInterface* FFastGeoPrimitiveComponent::GetScene() const
{
	return Super::GetScene();
}

bool FFastGeoPrimitiveComponent::IsFirstPersonRelevant() const
{
	return GetSceneProxyDesc().IsFirstPersonRelevant();
}

bool FFastGeoPrimitiveComponent::IsCollisionEnabled() const
{
	return GetSceneProxyDesc().bCollisionEnabled;
}

FPrimitiveSceneProxy* FFastGeoPrimitiveComponent::GetSceneProxy() const
{
	return PrimitiveSceneData.SceneProxy;
}

void FFastGeoPrimitiveComponent::MarkRenderStateDirty()
{
	Super::MarkRenderStateDirty(false);
}

void FFastGeoPrimitiveComponent::DestroyRenderState()
{
	DestroyRenderState(nullptr);
}

FTransform FFastGeoPrimitiveComponent::GetTransform() const
{
	return WorldTransform;
}

const FBoxSphereBounds& FFastGeoPrimitiveComponent::GetBounds() const
{
	return WorldBounds;
}

void FFastGeoPrimitiveComponent::GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const
{
}

UObject* FFastGeoPrimitiveComponent::GetUObject()
{
	return GetOwnerContainer();
}

const UObject* FFastGeoPrimitiveComponent::GetUObject() const
{
	return GetOwnerContainer();
}

FMatrix FFastGeoPrimitiveComponent::GetRenderMatrix() const
{
	return GetTransform().ToMatrixWithScale();
}

float FFastGeoPrimitiveComponent::GetLastRenderTimeOnScreen() const
{
	return PrimitiveSceneData.GetLastRenderTimeOnScreen();
}

void FFastGeoPrimitiveComponent::SetCollisionEnabled(bool bInCollisionEnabled)
{
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.bCollisionEnabled = bInCollisionEnabled;
}

void FFastGeoPrimitiveComponent::SetCustomPrimitiveData(TConstArrayView<float> InCustomPrimitiveData)
{
	// Make sure to clamp to the maximum number of values supported by custom primitive data.
	CustomPrimitiveData.Data = InCustomPrimitiveData.Left(FMath::Min(InCustomPrimitiveData.Num(), FCustomPrimitiveData::NumCustomPrimitiveDataFloats));
	MarkRenderStateDirty();
}

void FFastGeoPrimitiveComponent::InitializeDynamicProperties()
{
	Super::InitializeDynamicProperties();

#if !WITH_EDITOR
	BodyInstance.FixupData(GetOwnerContainer());
#endif
}

bool FFastGeoPrimitiveComponent::UsePSOPrecacheFallbackMaterial() const
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Skip fallback material during blocking wait -- force real material creation.
	if (GetOwnerContainer()->GetWorldSubsystem()->IsWaitingForCompletion())
	{
		return false;
	}
	return PSOPrecacheComponentData.IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached;
#else
	return false;
#endif
}

void FFastGeoPrimitiveComponent::PrecachePSOs()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	PrecachePSOs_Concurrent();
#endif
}

bool FFastGeoPrimitiveComponent::CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority)
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// When the world subsystem is in a blocking wait (e.g., BlockTillLevelStreamingCompleted),
	// skip the PSO delay check to create proxies immediately regardless of PSO compilation
	// state. GT is blocked and PSO callbacks cannot fire, so we force proxy creation to
	// match the AddToWorld contract (all proxies created when blocking completes).
	if (GetOwnerContainer()->GetWorldSubsystem()->IsWaitingForCompletion())
	{
		return false;
	}
	return PSOPrecacheComponentData.CheckPSOPrecachingAndBoostPriority(NewPSOPrecachePriority);
#else
	return false;
#endif
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FFastGeoPrimitiveComponent::SetupPrecachePSOParams(FPSOPrecacheParams& Params)
{
	const FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	//auto IsPrecomputedLightingValid = []() { return false; };
	Params.bRenderInMainPass = SceneProxyDesc.bRenderInMainPass;
	Params.bRenderInDepthPass = SceneProxyDesc.bRenderInDepthPass;
	Params.bStaticLighting = SceneProxyDesc.bHasStaticLighting;
	Params.bUsesIndirectLightingCache = Params.bStaticLighting && SceneProxyDesc.IndirectLightingCacheQuality != ILCQ_Off /*&& (!IsPrecomputedLightingValid() || SceneProxyDesc.LightmapType == ELightmapType::ForceVolumetric)*/;
	Params.bAffectDynamicIndirectLighting = SceneProxyDesc.bAffectDynamicIndirectLighting;
	Params.bCastShadow = SceneProxyDesc.CastShadow;
	// Custom depth can be toggled at runtime with PSO precache call so assume it might be needed when depth pass is needed
	// Ideally precache those with lower priority and don't wait on these (UE-174426)
	Params.bRenderCustomDepth = SceneProxyDesc.bRenderInDepthPass;
	Params.bCastShadowAsTwoSided = SceneProxyDesc.bCastShadowAsTwoSided;
	Params.SetMobility(SceneProxyDesc.Mobility);
	Params.SetStencilWriteMask(FRendererStencilMaskEvaluation::ToStencilMask(SceneProxyDesc.CustomDepthStencilWriteMask));

	TArray<UMaterialInterface*> UsedMaterials;
	GetUsedMaterials(UsedMaterials);
	for (const UMaterialInterface* MaterialInterface : UsedMaterials)
	{
		if (MaterialInterface && MaterialInterface->IsUsingWorldPositionOffset_Concurrent(GMaxRHIShaderPlatform))
		{
			Params.bAnyMaterialHasWorldPositionOffset = true;
			break;
		}
	}
}

void FFastGeoPrimitiveComponent::PrecachePSOs_Concurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFastGeoPrimitiveComponent::PrecachePSOs_Concurrent);
	if (!ShouldPrecachePSOs())
	{
		return;
	}

	// Collect the data from the derived classes
	FPSOPrecacheParams PSOPrecacheParams;
	SetupPrecachePSOParams(PSOPrecacheParams);

	FMaterialInterfacePSOPrecacheParamsList PSOPrecacheDataArray;
	CollectPSOPrecacheData(PSOPrecacheParams, PSOPrecacheDataArray);

	FGraphEventArray PSOPrecacheCompileEvents;
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	PrecacheMaterialPSOs(PSOPrecacheDataArray, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents);

	auto OnPrecacheFinishedGT = [WeakComponent = FWeakFastGeoComponent(this)]()
	{
		if (FFastGeoPrimitiveComponent* Component = WeakComponent.Get<FFastGeoPrimitiveComponent>())
		{
			if (UFastGeoContainer* Container = Component->GetOwnerContainer())
			{
				if (Container->GetWorld())
				{
					Container->OnComponentPSOPrecacheCompleted(Component);
				}
			}
		}
	};

	PSOPrecacheComponentData.SetMaterialPSOPrecacheData(this, MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents, OnPrecacheFinishedGT);
}
#endif

UObject* FFastGeoPrimitiveComponent::GetOwner() const
{
	return GetOwnerContainer();
}

FString FFastGeoPrimitiveComponent::GetOwnerName() const
{
	return GetNameSafe(GetOwnerContainer());
}

FPrimitiveSceneProxy* FFastGeoPrimitiveComponent::CreateSceneProxy()
{
	return CreateSceneProxy(/*OutError*/nullptr);
}

//
// Functions below here added for use in streamer, which is not used by FastGeo. These functions may not get used.
static FRenderAssetOwnerStreamingState GFastGeoStreamingStateStub;
FRenderAssetOwnerStreamingState& FFastGeoPrimitiveComponent::GetStreamingState() const
{
	return GFastGeoStreamingStateStub;
}

ULevel* FFastGeoPrimitiveComponent::GetComponentLevel() const
{
	return nullptr;
}

IPrimitiveComponent* FFastGeoPrimitiveComponent::GetLODParentPrimitive() const
{
	return nullptr;
}

float FFastGeoPrimitiveComponent::GetMinDrawDistance() const
{
	return 0.0f;
}

float FFastGeoPrimitiveComponent::GetStreamingScale() const
{
	return 1.0f;
}

void FFastGeoPrimitiveComponent::OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex)
{
}

UStreamableRenderAsset* FFastGeoPrimitiveComponent::GetStreamableNaniteAsset() const
{
	return nullptr;
}

void FFastGeoPrimitiveComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
}

#if WITH_EDITOR
HHitProxy* FFastGeoPrimitiveComponent::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex)
{
	return nullptr;
}
#endif // WITH_EDITOR

HHitProxy* FFastGeoPrimitiveComponent::CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
	return nullptr;
}

bool FFastGeoPrimitiveComponent::IsNavigationRelevant() const
{
	if (!bCanEverAffectNavigation)
	{
		return false;
	}

	if (HasCustomNavigableGeometry() >= EHasCustomNavigableGeometry::EvenIfNotCollidable)
	{
		return true;
	}

	auto GetCollisionEnabled = [this]() -> ECollisionEnabled::Type
	{
		if (!IsCollisionEnabled())
		{
			return ECollisionEnabled::NoCollision;
		}
		return BodyInstance.GetCollisionEnabled(false);
	};

	auto IsQueryCollisionEnabled = [&GetCollisionEnabled]()
	{
		return CollisionEnabledHasQuery(GetCollisionEnabled());
	};

	auto GetCollisionResponseToChannels = [this]()
	{
		return BodyInstance.GetResponseToChannels();
	};

	const FCollisionResponseContainer& ResponseToChannels = GetCollisionResponseToChannels();
	return IsQueryCollisionEnabled() &&
		(ResponseToChannels.GetResponse(ECC_Pawn) == ECR_Block || ResponseToChannels.GetResponse(ECC_Vehicle) == ECR_Block);
}

FBox FFastGeoPrimitiveComponent::GetNavigationBounds() const
{
	return GetBounds().GetBox();
}

void FFastGeoPrimitiveComponent::GetNavigationData(FNavigationRelevantData& OutData) const
{
	FPrimitiveComponentHelper::GetNavigationData(*this, OutData);
}

EHasCustomNavigableGeometry::Type FFastGeoPrimitiveComponent::HasCustomNavigableGeometry() const
{
	return bHasCustomNavigableGeometry;
}

bool FFastGeoPrimitiveComponent::IsStaticPhysics() const
{
	return GetSceneProxyDesc().Mobility != EComponentMobility::Movable && bStaticWhenNotMoveable;
}

bool FFastGeoPrimitiveComponent::IsMultiBodyOverlap() const
{
	return bMultiBodyOverlap;
}

UObject* FFastGeoPrimitiveComponent::GetSourceObject() const
{
	return GetOwnerContainer();
}

ECollisionResponse FFastGeoPrimitiveComponent::GetCollisionResponseToChannel(ECollisionChannel Channel) const
{
	return BodyInstance.GetResponseToChannel(Channel);
}

// Deprecated in 5.7
FPrimitiveMaterialPropertyDescriptor FFastGeoPrimitiveComponent::GetUsedMaterialPropertyDesc(ERHIFeatureLevel::Type FeatureLevel) const
{
	return GetUsedMaterialPropertyDesc(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
}

FPrimitiveMaterialPropertyDescriptor FFastGeoPrimitiveComponent::GetUsedMaterialPropertyDesc(EShaderPlatform InShaderPlatform) const
{
	return FPrimitiveComponentHelper::GetUsedMaterialPropertyDesc(*this, InShaderPlatform);
}

ECollisionChannel FFastGeoPrimitiveComponent::GetCollisionObjectType() const
{
	return ECollisionChannel(BodyInstance.GetObjectType());
}

ECollisionEnabled::Type FFastGeoPrimitiveComponent::GetCollisionEnabled() const
{
	if (!IsCollisionEnabled())
	{
		return ECollisionEnabled::NoCollision;
	}

	return BodyInstance.GetCollisionEnabled(false);
}

Chaos::FPhysicsObject* FFastGeoPrimitiveComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!BodyInstance.IsValidBodyInstance())
	{
		return nullptr;
	}

	return BodyInstance.GetPhysicsActor()->GetPhysicsObject();
}

TArray<Chaos::FPhysicsObject*> FFastGeoPrimitiveComponent::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObject*> Bodies;
	if (Chaos::FPhysicsObject* PhysicsObject = GetPhysicsObjectById(INDEX_NONE))
	{
		Bodies.Add(PhysicsObject);
	}
	return Bodies;
}

UPhysicalMaterial* FFastGeoPrimitiveComponent::GetPhysicsMaterialOverride() const
{
	return BodyInstance.GetPhysMaterialOverride();
}

FBodyInstance* FFastGeoPrimitiveComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	if (Index != INDEX_NONE)
	{
		// Handle the case where GetBodyInstance is called directly on the FastGeo component with an already transformed Index
		if (UFastGeoSurrogateComponent* SurrogateComponent = GetSurrogateComponent())
		{
			if (FFastGeoSurrogateBodyInstanceIndex::IsEncoded(Index))
			{
				return SurrogateComponent->GetBodyInstance(BoneName, bGetWelded, Index);
			}
		}
	}
	return const_cast<FBodyInstance*>(&BodyInstance);
}

void FFastGeoPrimitiveComponent::OnAsyncCreatePhysicsStateBegin_GameThread()
{
	BodyInstanceOwner.Initialize(this);
	BodyInstance.CachePhysicsCreationInputs(GetSurrogateComponent(), &BodyInstanceOwner);
	BodyInstanceOwner.Uninitialize();

	Super::OnAsyncCreatePhysicsStateBegin_GameThread();
}

void FFastGeoPrimitiveComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	BodyInstance.ClearCachedPhysicsCreationInputs();

	Super::OnAsyncCreatePhysicsStateEnd_GameThread();
}

UFastGeoSurrogateComponent* FFastGeoPrimitiveComponent::GetSurrogateComponent() const
{
	// Components with collision disabled are not assigned a surrogate descriptor
	// index during cook-time transformation (see FastGeoWorldPartitionRuntimeCellTransformer).
	if (SurrogateComponentDescriptorIndex == INDEX_NONE)
	{
		return nullptr;
	}

	if (UFastGeoContainer* Container = GetOwnerContainer(); ensure(Container))
	{
		if (Container->IsUsingSurrogateComponents())
		{
			if (const AFastGeoSurrogateActor* SurrogateActor = Container->GetSurrogateActor())
			{
				UFastGeoSurrogateComponent* Component = SurrogateActor->GetSurrogateComponent(SurrogateComponentDescriptorIndex);
				ensure(Component);
				return Component;
			}
		}
	}
	return nullptr;
}

#if WITH_EDITOR
void FFastGeoPrimitiveComponent::ReserveSurrogateInstanceBodyIndices(FFastGeoSurrogateBodyInstanceIndex& InOutNextAvailableInstanceBodyIndex)
{
	check(BodyInstance.InstanceBodyIndex == INDEX_NONE);
	BodyInstance.InstanceBodyIndex = InOutNextAvailableInstanceBodyIndex.GetEncoded();
	check(FFastGeoSurrogateBodyInstanceIndex::IsEncoded(BodyInstance.InstanceBodyIndex));

	++InOutNextAvailableInstanceBodyIndex;
}
#endif