// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoProceduralISMComponent.h"

#include "InstancedStaticMeshComponentHelper.h"
#include "NaniteSceneProxy.h"
#include "PrimitiveSceneDesc.h"
#include "InstancedStaticMesh/ISMInstanceDataSceneProxy.h"
#include "ISMPartition/ProceduralISMComponentDescriptor.h"
#if WITH_EDITOR
#include "EngineUtils.h"
#endif

const FFastGeoElementType FFastGeoProceduralISMComponent::Type(&FFastGeoStaticMeshComponentBase::Type);

FFastGeoProceduralISMComponent::FFastGeoProceduralISMComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

void FFastGeoProceduralISMComponent::InitializeFromComponentDescriptor(const FProceduralISMComponentDescriptor& InDescriptor)
{
	NumInstances = InDescriptor.NumInstances;
	NumCustomDataFloats = InDescriptor.NumCustomFloats;
	PrimitiveBoundsOverride = InDescriptor.WorldBounds;
	LocalBounds = PrimitiveBoundsOverride.InverseTransformBy(WorldTransform);
	WorldBounds = InDescriptor.WorldBounds;
	OverrideMaterials = InDescriptor.OverrideMaterials;
	RuntimeVirtualTextures = InDescriptor.RuntimeVirtualTextures;

	SceneProxyDesc.StaticMesh = InDescriptor.StaticMesh.Get();
	SceneProxyDesc.OverlayMaterial = InDescriptor.OverlayMaterial;
	SceneProxyDesc.MinDrawDistance = InDescriptor.InstanceMinDrawDistance;
	SceneProxyDesc.InstanceStartCullDistance = InDescriptor.InstanceStartCullDistance;
	SceneProxyDesc.InstanceEndCullDistance = InDescriptor.InstanceEndCullDistance;
	SceneProxyDesc.Mobility = InDescriptor.Mobility;
	SceneProxyDesc.VirtualTextureRenderPassType = InDescriptor.VirtualTextureRenderPassType;
	SceneProxyDesc.LightingChannels = InDescriptor.LightingChannels;
	SceneProxyDesc.CustomDepthStencilWriteMask = InDescriptor.CustomDepthStencilWriteMask;
	SceneProxyDesc.VirtualTextureCullMips = InDescriptor.VirtualTextureCullMips;
	SceneProxyDesc.TranslucencySortPriority = InDescriptor.TranslucencySortPriority;
	SceneProxyDesc.CustomDepthStencilValue = InDescriptor.CustomDepthStencilValue;
	SceneProxyDesc.bVisibleInRayTracing = InDescriptor.bVisibleInRayTracing;
	SceneProxyDesc.bVisibleInReflections = InDescriptor.bVisibleInReflections;
	SceneProxyDesc.RayTracingGroupId = InDescriptor.RayTracingGroupId;
	SceneProxyDesc.RayTracingGroupCullingPriority = InDescriptor.RayTracingGroupCullingPriority;
	SceneProxyDesc.CastShadow = InDescriptor.bCastShadow;
	SceneProxyDesc.bEmissiveLightSource = InDescriptor.bEmissiveLightSource;
	SceneProxyDesc.bCastDynamicShadow = InDescriptor.bCastDynamicShadow;
	SceneProxyDesc.bCastStaticShadow = InDescriptor.bCastStaticShadow;
	SceneProxyDesc.bCastContactShadow = InDescriptor.bCastContactShadow;
	SceneProxyDesc.bCastShadowAsTwoSided = InDescriptor.bCastShadowAsTwoSided;
	SceneProxyDesc.bCastHiddenShadow = InDescriptor.bCastHiddenShadow;
	SceneProxyDesc.bReceivesDecals = InDescriptor.bReceivesDecals;
	SceneProxyDesc.bUseAsOccluder = InDescriptor.bUseAsOccluder;
	SceneProxyDesc.bRenderCustomDepth = InDescriptor.bRenderCustomDepth;
	SceneProxyDesc.bEvaluateWorldPositionOffset = InDescriptor.bEvaluateWorldPositionOffset;
	SceneProxyDesc.bReverseCulling = InDescriptor.bReverseCulling;
	SceneProxyDesc.WorldPositionOffsetDisableDistance = InDescriptor.WorldPositionOffsetDisableDistance;
	SceneProxyDesc.ShadowCacheInvalidationBehavior = InDescriptor.ShadowCacheInvalidationBehavior;
	SceneProxyDesc.bIsInstancedStaticMesh = true;
}

void FFastGeoProceduralISMComponent::SetSpatialHashes(TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>&& InSpatialHashes)
{
	if (SpatialHashes != InSpatialHashes)
	{
		SpatialHashes = MoveTemp(InSpatialHashes);
		MarkRenderStateDirty();
	}
}

void FFastGeoProceduralISMComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FFastGeoProceduralISMComponent
	Ar << NumInstances;
	Ar << NumCustomDataFloats;
	Ar << PrimitiveBoundsOverride;

	// Serialize persistent data from FInstancedStaticMeshSceneProxyDesc
	Ar << SceneProxyDesc.InstanceLODDistanceScale;
	Ar << SceneProxyDesc.InstanceMinDrawDistance;
	Ar << SceneProxyDesc.InstanceStartCullDistance;
	Ar << SceneProxyDesc.InstanceEndCullDistance;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bUseGpuLodSelection);
}

FPrimitiveSceneDesc FFastGeoProceduralISMComponent::BuildSceneDesc()
{
	FPrimitiveSceneDesc SceneDesc = Super::BuildSceneDesc();
#if WITH_EDITOR
	SceneDesc.PrimitiveComponentInterface = this;
#endif

	return SceneDesc;
}

#if WITH_EDITOR
void FFastGeoProceduralISMComponent::SetHitProxyTargetActor(AActor* InActor)
{
	checkf(!IsRegistered(), TEXT("SetHitProxyTargetActor must be called before Register()"));
	HitProxyTargetActor = InActor;
}

void FFastGeoProceduralISMComponent::UpdateEditorSelectionState()
{
	if (AActor* Actor = HitProxyTargetActor.Get())
	{
		if (FPrimitiveSceneProxy* Proxy = GetSceneProxy())
		{
			Proxy->SetSelection_GameThread(Actor->IsSelected(), /*bInIndividuallySelected=*/false);
		}
	}
}

HHitProxy* FFastGeoProceduralISMComponent::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex)
{
	if (AActor* Actor = HitProxyTargetActor.Get())
	{
		return new HActor(Actor, /*InPrimComponent=*/nullptr);
	}
	return nullptr;
}

void FFastGeoProceduralISMComponent::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	SceneProxyDesc.InstanceDataSceneProxy = nullptr;
	SceneProxyDesc.bHasSelectedInstances = false;
	SceneProxyDesc.bAffectDynamicIndirectLighting = false;
	SceneProxyDesc.bAffectDistanceFieldLighting = false;
	SceneProxyDesc.bCollisionEnabled = false;
}
#endif

HHitProxy* FFastGeoProceduralISMComponent::CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
#if WITH_EDITOR
	if (AActor* Actor = HitProxyTargetActor.Get())
	{
		HHitProxy* Proxy = new HActor(Actor, /*InPrimComponent=*/nullptr);
		OutHitProxies.Add(Proxy);
		return Proxy;
	}
#endif
	return nullptr;
}

void FFastGeoProceduralISMComponent::InitializeSceneProxyDescDynamicProperties()
{
	Super::InitializeSceneProxyDescDynamicProperties();

	SceneProxyDesc.bAffectDynamicIndirectLighting = false;
	SceneProxyDesc.bAffectDistanceFieldLighting = false;
	SceneProxyDesc.bCollisionEnabled = false;

#if WITH_EDITOR
	// ResetSceneProxyDescUnsupportedProperties (called from Super above) marks bSelectable=false for all FastGeo
	// primitives. Re-enable it if a hit proxy target actor was set so that this primitive is rendered into
	// the hit proxy buffer and can be clicked in the editor.
	if (HitProxyTargetActor.IsValid())
	{
		SceneProxyDesc.bSelectable = true;
		SceneProxyDesc.bSelected = HitProxyTargetActor->IsSelected();
	}
#endif
}

void FFastGeoProceduralISMComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	// Procedural component differs - bounds cannot be computed from instance data since it is not available on CPU,
	// and bounds are provided by user in world space. Apply transform, then recompute local bounds from that.
	FFastGeoPrimitiveComponent::ApplyWorldTransform(InTransform);

	LocalBounds = PrimitiveBoundsOverride.InverseTransformBy(WorldTransform);
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe>& FFastGeoProceduralISMComponent::BuildInstanceData()
{
	FInstanceSceneDataBuffers InstanceSceneDataBuffers(/*InbInstanceDataIsGPUOnly=*/true);
	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	auto View = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
	
	// PrimitiveLocalToWorld
	InstanceSceneDataBuffers.SetPrimitiveLocalToWorld(GetRenderMatrix(), AccessTag);
	
	// InstanceLocalBounds
	const FPrimitiveMaterialPropertyDescriptor PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(GetScene()->GetShaderPlatform());
	const float LocalAbsMaxDisplacement = FMath::Max(-PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y) + PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;
	const FVector3f PadExtent = FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(View.PrimitiveToRelativeWorld, LocalAbsMaxDisplacement);
	FRenderBounds InstanceLocalBounds = GetStaticMesh()->GetBounds();
	InstanceLocalBounds.Min -= PadExtent;
	InstanceLocalBounds.Max += PadExtent;
	check(!View.Flags.bHasPerInstanceLocalBounds);
	View.InstanceLocalBounds.Add(InstanceLocalBounds);

	View.NumInstancesGPUOnly = NumInstances;
	View.NumCustomDataFloats = NumCustomDataFloats;
	View.Flags.bHasPerInstanceCustomData = NumCustomDataFloats > 0;
	View.Flags.bHasPerInstanceRandom = PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom;

	if (!SpatialHashes.IsEmpty())
	{
		InstanceSceneDataBuffers.SetImmutable(FInstanceSceneDataImmutable(SpatialHashes), AccessTag);
	}

	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	InstanceSceneDataBuffers.ValidateData();

	DataProxy = MakeShared<FInstanceDataSceneProxy, ESPMode::ThreadSafe>(MoveTemp(InstanceSceneDataBuffers));
	return DataProxy;
}

FPrimitiveSceneProxy* FFastGeoProceduralISMComponent::CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	check(GetWorld());
	check(SceneProxyDesc.Scene);
	check(NumInstances > 0);

	SceneProxyDesc.InstanceDataSceneProxy = BuildInstanceData();
	if (bCreateNanite)
	{
		PrimitiveSceneData.SceneProxy = ::new Nanite::FSceneProxy(NaniteMaterials, SceneProxyDesc);
	}
	else
	{
		PrimitiveSceneData.SceneProxy = ::new FInstancedStaticMeshSceneProxy(SceneProxyDesc, SceneProxyDesc.FeatureLevel);
	}
	return PrimitiveSceneData.SceneProxy;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FFastGeoProceduralISMComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	return FInstancedStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}
#endif
