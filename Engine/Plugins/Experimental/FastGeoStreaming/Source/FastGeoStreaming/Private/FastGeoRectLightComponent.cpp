// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoRectLightComponent.h"
#include "Components/RectLightComponent.h"
#include "RectLightSceneProxy.h"
#include "RectLightSceneProxyDesc.h"
#include "SceneInterface.h"

const FFastGeoElementType FFastGeoRectLightComponent::Type(&FFastGeoLocalLightComponent::Type);

FFastGeoRectLightComponent::FFastGeoRectLightComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

#if WITH_EDITOR
void FFastGeoRectLightComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	URectLightComponent* RectLightComponent = CastChecked<URectLightComponent>(Component);
	SceneProxyDesc = URectLightComponent::BuildSceneProxyDesc(*RectLightComponent);
	SceneProxyDesc.LightComponent = nullptr;
}
#endif

FLightSceneProxy* FFastGeoRectLightComponent::CreateTypedSceneProxy()
{
	check(SceneProxyDesc.SceneInterface == GetScene());
	return new FRectLightSceneProxy(SceneProxyDesc);
}