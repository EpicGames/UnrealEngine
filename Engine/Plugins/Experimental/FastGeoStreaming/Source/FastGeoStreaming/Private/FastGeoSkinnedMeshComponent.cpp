// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoSkinnedMeshComponent.h"
#include "ContentStreaming.h"
#include "Engine/MaterialOverlayHelper.h"
#include "FastGeoHLOD.h"
#include "FastGeoLog.h"
#include "HAL/LowLevelMemStats.h"
#include "Materials/MaterialInterface.h"
#include "NaniteSceneProxy.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"
#include "SkeletalRenderPublic.h"
#include "SkinnedMeshComponentHelper.h"

const FFastGeoElementType FFastGeoSkinnedMeshComponentBase::Type(&FFastGeoMeshComponent::Type);

FFastGeoSkinnedMeshComponentBase::FFastGeoSkinnedMeshComponentBase(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

void FFastGeoSkinnedMeshComponentBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FFastGeoStaticMeshComponentBase
	Ar << SkinCacheUsage;
	FArchive_Serialize_BitfieldBool(Ar, bOverrideMinLod);
	FArchive_Serialize_BitfieldBool(Ar, bIncludeComponentLocationIntoBounds);
	FArchive_Serialize_BitfieldBool(Ar, bHideSkin);	
	Ar << MinLodModel;
	
	// Serialize persistent data from FStaticMeshSceneProxyDesc
	FSkinnedMeshSceneProxyDesc& SceneProxyDesc = GetSkinnedMeshSceneProxyDesc();
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bForceWireframe);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCanHighlightSelectedSections);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bRenderStatic);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bPerBoneMotionBlur);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastCapsuleDirectShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastCapsuleIndirectShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCPUSkinning);
	Ar << SceneProxyDesc.StreamingDistanceMultiplier;
	Ar << SceneProxyDesc.NanitePixelProgrammableDistance;
	Ar << SceneProxyDesc.CapsuleIndirectShadowMinVisibility;
	Ar << SceneProxyDesc.OverlayMaterialMaxDrawDistance;
	Ar << SceneProxyDesc.PredictedLODLevel;
	Ar << SceneProxyDesc.ForceStreamedLOD;
	Ar << SceneProxyDesc.MaxDistanceFactor;
	Ar << SceneProxyDesc.ComponentScale;
	Ar << SceneProxyDesc.SkinnedAsset;
	Ar << SceneProxyDesc.OverlayMaterial;
	Ar << SceneProxyDesc.MaterialSlotsOverlayMaterial;
}

UBodySetup* FFastGeoSkinnedMeshComponentBase::GetBodySetup() const
{
	return nullptr;
}

#if WITH_EDITOR
void FFastGeoSkinnedMeshComponentBase::InitializeSceneProxyDescFromComponent(UActorComponent* Component)
{
	USkinnedMeshComponent* SkinnedMeshComponent = CastChecked<USkinnedMeshComponent>(Component);
	FSkinnedMeshSceneProxyDesc& SceneProxyDesc = GetSkinnedMeshSceneProxyDesc();
	SceneProxyDesc.InitializeFromSkinnedMeshComponent(SkinnedMeshComponent);
}

void FFastGeoSkinnedMeshComponentBase::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	USkinnedMeshComponent* SkinnedMeshComponent = CastChecked<USkinnedMeshComponent>(Component);
	SkinCacheUsage = SkinnedMeshComponent->SkinCacheUsage;
	bOverrideMinLod = SkinnedMeshComponent->bOverrideMinLod;
	bIncludeComponentLocationIntoBounds = SkinnedMeshComponent->bIncludeComponentLocationIntoBounds;
	bHideSkin = SkinnedMeshComponent->bHideSkin;	
	MinLodModel = SkinnedMeshComponent->MinLodModel;

	LocalBounds = SkinnedMeshComponent->CalcBounds(FTransform::Identity);
	WorldBounds = SkinnedMeshComponent->CalcBounds(WorldTransform);
}

void FFastGeoSkinnedMeshComponentBase::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	// Unsupported properties
	FSkinnedMeshSceneProxyDesc& SceneProxyDesc = GetSkinnedMeshSceneProxyDesc();
#if WITH_EDITORONLY_DATA
	SceneProxyDesc.bClothPainting = false;
	check(SceneProxyDesc.GetSectionPreview() == INDEX_NONE);
	check(SceneProxyDesc.GetMaterialPreview() == INDEX_NONE);
	check(SceneProxyDesc.GetSelectedEditorSection() == INDEX_NONE);
	check(SceneProxyDesc.GetSelectedEditorMaterial() == INDEX_NONE);
#endif
#if UE_ENABLE_DEBUG_DRAWING
	SceneProxyDesc.bDrawDebugSkeleton = false;
	SceneProxyDesc.DebugDrawColor.Reset();
#endif

	// Properties that will be initialized by InitializeSceneProxyDescDynamicProperties
	SceneProxyDesc.MaterialRelevance = FMaterialRelevance();
}
#endif

void FFastGeoSkinnedMeshComponentBase::InitializeSceneProxyDescDynamicProperties()
{
	Super::InitializeSceneProxyDescDynamicProperties();

	// Destroy any existing mesh object
	DestroyMeshObject();

	// Create mesh object
	MeshObject = CreateMeshObject();

	// Initialize non-serialized properties
	FSkinnedMeshSceneProxyDesc& SceneProxyDesc = GetSkinnedMeshSceneProxyDesc();
	SceneProxyDesc.OverrideMaterials = OverrideMaterials;
	SceneProxyDesc.SkinCacheUsage = SkinCacheUsage;
	SceneProxyDesc.MaterialRelevance = GetMaterialRelevance(GetScene()->GetShaderPlatform());
	SceneProxyDesc.MeshObject = MeshObject;
	SceneProxyDesc.MinLODLevel = FSkinnedMeshComponentHelper::ComputeMinLOD(*this);
}

FPrimitiveSceneProxy* FFastGeoSkinnedMeshComponentBase::CreateSceneProxy(ESceneProxyCreationError* OutError)
{
	check(GetWorld());
	FSceneInterface* Scene = GetScene();
	check(Scene);

	USkinnedAsset* SkinnedAsset = GetSkinnedAsset();
	check(SkinnedAsset);
	check(!SkinnedAsset->IsCompiling());
	
	if (OutError)
	{
		*OutError = ESceneProxyCreationError::None;
	}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogFastGeoStreaming, Verbose, "Skipping CreateSceneProxy for FastGeoSkinnedMeshComponent of %ls (component PSOs are still compiling)", *GetOwnerComponentCluster()->GetName());
		if (OutError)
		{
			*OutError = ESceneProxyCreationError::WaitingPSOs;
		}
		return nullptr;
	}
#endif

	InitializeSceneProxyDescDynamicProperties();

	FSkeletalMeshRenderData* SkelMeshRenderData = FSkinnedMeshComponentHelper::GetSkeletalMeshRenderData(*this);
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->IsInitialized());

	FSkinnedMeshSceneProxyDesc& SceneProxyDesc = GetSkinnedMeshSceneProxyDesc();
	check(SceneProxyDesc.Scene);
	check(SceneProxyDesc.Scene == Scene);
	check(SceneProxyDesc.World == Scene->GetWorld());
	check(SceneProxyDesc.FeatureLevel == Scene->GetFeatureLevel());
	check(SceneProxyDesc.ComponentId == GetPrimitiveSceneId());
	check(SceneProxyDesc.MeshObject);

	LLM_SCOPE(ELLMTag::SkeletalMesh);
	ERHIFeatureLevel::Type SceneFeatureLevel = GetWorld()->GetFeatureLevel();
	
	PrimitiveSceneData.SceneProxy = AllocateSceneProxy();

	UpdateSkinning();

	return PrimitiveSceneData.SceneProxy;
}

void FFastGeoSkinnedMeshComponentBase::DestroyMeshObject()
{
	if (MeshObject)
	{
		// ReleaseResources is safe from any thread (MPSC queues, ENQUEUE_RENDER_COMMAND).
		MeshObject->ReleaseResources();

		// BeginCleanup must be called from the game thread or parallel game thread.
		if (IsInGameThread() || IsInParallelGameThread())
		{
			BeginCleanup(MeshObject);
		}
		else
		{
			FSkeletalMeshObject* MeshObjectToCleanup = MeshObject;
			AsyncTask(ENamedThreads::GameThread, [MeshObjectToCleanup]()
			{
				BeginCleanup(MeshObjectToCleanup);
			});
		}

		MeshObject = nullptr;
	}
}

void FFastGeoSkinnedMeshComponentBase::DestroyRenderState(FFastGeoDestroyRenderStateContext* Context)
{
	DestroyMeshObject();

	Super::DestroyRenderState(Context);
}

void FFastGeoSkinnedMeshComponentBase::ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const
{
	auto AddMaterial = [&Func](UMaterialInterface* Mat)
	{
		Func(Mat, false);
		if (UMaterialInterface* NaniteOverride = Mat->GetNaniteOverride())
		{
			Func(NaniteOverride, true);
		}
	};

	for (const TObjectPtr<UMaterialInterface>& Mat : OverrideMaterials)
	{
		if (Mat)
		{
			AddMaterial(Mat);
		}
	}

	if (USkinnedAsset* Asset = GetSkinnedAsset())
	{
		for (const FSkeletalMaterial& SlotMat : Asset->GetMaterials())
		{
			if (SlotMat.MaterialInterface)
			{
				AddMaterial(SlotMat.MaterialInterface);
			}
			if (SlotMat.OverlayMaterialInterface)
			{
				AddMaterial(SlotMat.OverlayMaterialInterface);
			}
		}
	}

	// Component overlay material
	const FSkinnedMeshSceneProxyDesc& Desc = GetSkinnedMeshSceneProxyDesc();
	if (Desc.OverlayMaterial)
	{
		AddMaterial(Desc.OverlayMaterial);
	}

	// Component per-slot overlay materials
	for (const TObjectPtr<UMaterialInterface>& Mat : Desc.MaterialSlotsOverlayMaterial)
	{
		if (Mat)
		{
			AddMaterial(Mat);
		}
	}
}

UMaterialInterface* FFastGeoSkinnedMeshComponentBase::GetMaterial(int32 MaterialIndex) const
{
	return FSkinnedMeshComponentHelper::GetMaterial(*this, MaterialIndex);
}

int32 FFastGeoSkinnedMeshComponentBase::GetNumMaterials() const
{
	if (GetSkinnedAsset() && !GetSkinnedAsset()->IsCompiling())
	{
		return GetSkinnedAsset()->GetMaterials().Num();
	}
	return 0;
}

void FFastGeoSkinnedMeshComponentBase::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	FSkinnedMeshComponentHelper::GetUsedMaterials(*this, OutMaterials, bGetDebugMaterials);
}

UMaterialInterface* FFastGeoSkinnedMeshComponentBase::GetOverlayMaterial() const
{
	return GetSkinnedMeshSceneProxyDesc().OverlayMaterial;
}

const TArray<TObjectPtr<UMaterialInterface>>& FFastGeoSkinnedMeshComponentBase::GetComponentMaterialSlotsOverlayMaterial() const
{
	return GetSkinnedMeshSceneProxyDesc().MaterialSlotsOverlayMaterial;
}

void FFastGeoSkinnedMeshComponentBase::GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& AssetMaterialSlotOverlayMaterials) const
{
	FSkinnedMeshComponentHelper::GetDefaultMaterialSlotsOverlayMaterial(*this, AssetMaterialSlotOverlayMaterials);
}

FBoxSphereBounds FFastGeoSkinnedMeshComponentBase::CalcMeshBound(const FVector3f& InRootOffset, bool bInUsePhysicsAsset, const FTransform& InLocalToWorld) const
{
	FBoxSphereBounds MeshBounds;

	if (GetSkinnedAsset())
	{
		FBoxSphereBounds RootAdjustedBounds = GetSkinnedAsset()->GetBounds();

		// Adjust bounds by root bone translation
		RootAdjustedBounds.Origin += FVector(InRootOffset);
		MeshBounds = RootAdjustedBounds.TransformBy(InLocalToWorld);
	}
	else
	{
		MeshBounds = FBoxSphereBounds(InLocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}

	MeshBounds.BoxExtent *= GetSceneProxyDesc().BoundsScale;
	MeshBounds.SphereRadius *= GetSceneProxyDesc().BoundsScale;

	return MeshBounds;
}

FSkeletalMeshObject* FFastGeoSkinnedMeshComponentBase::CreateMeshObject()
{
	return FSkinnedMeshSceneProxyDesc::CreateMeshObject(GetSkinnedMeshSceneProxyDesc());
}

void FFastGeoSkinnedMeshComponentBase::UpdateSkinning()
{
	// Non-instanced FSkeletalMeshObject paths (GPUSkin, Static, CPU skin) consume the bone arrays
	// in Update - e.g. GPU skin uploads them as the per-frame pose. FastGeo doesn't tick animation,
	// so feed the asset's ref pose. The instanced subclass overrides this method to skip the fill.
	TArray<FTransform> ComponentSpaceRefPose;
	FAnimationRuntime::FillUpComponentSpaceTransforms(GetSkinnedAsset()->GetRefSkeleton(), GetSkinnedAsset()->GetRefSkeleton().GetRefBonePose(), ComponentSpaceRefPose);

	FSkinnedMeshSceneProxyDynamicData DynamicData;
	DynamicData.ComponentSpaceTransforms = ComponentSpaceRefPose;
	DynamicData.PreviousComponentSpaceTransforms = ComponentSpaceRefPose;
	DynamicData.bRenderStateRecreating = true;

	RunMeshObjectUpdate(DynamicData);
}

void FFastGeoSkinnedMeshComponentBase::RunMeshObjectUpdate(FSkinnedMeshSceneProxyDynamicData& InOutDynamicData)
{
	const int32 PredictedLOD = GetPredictedLODLevel();
	const int32 MinLodIndex = FSkinnedMeshComponentHelper::ComputeMinLOD(*this);
	const int32 MaxLODIndex = MeshObject->GetSkeletalMeshRenderData().LODRenderData.Num() - 1;
	int32 UseLOD = FMath::Clamp(PredictedLOD, MinLodIndex, MaxLODIndex);

	// Clamp to loaded streaming data if available
	if (GetSkinnedAsset()->IsStreamable() || !IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh))
	{
		UseLOD = FMath::Max<int32>(UseLOD, MeshObject->GetSkeletalMeshRenderData().PendingFirstLODIdx);
	}

	// Guard against asset/render-data inconsistency. Mirrors USkinnedMeshComponent::SendRenderDynamicData_Concurrent.
	if (!GetSkinnedAsset()->IsValidLODIndex(UseLOD))
	{
		return;
	}

	MeshObject->Update(
		UseLOD,
		InOutDynamicData,
		PrimitiveSceneData.SceneProxy,
		GetSkinnedAsset(),
		FMorphTargetWeightMap(),
		TArray<float>(),
		EPreviousBoneTransformUpdateMode::UpdatePrevious,
		FExternalMorphWeightData()
	);
}

const FFastGeoElementType FFastGeoSkinnedMeshComponent::Type(&FFastGeoSkinnedMeshComponentBase::Type);

FFastGeoSkinnedMeshComponent::FFastGeoSkinnedMeshComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

FPrimitiveSceneProxy* FFastGeoSkinnedMeshComponent::AllocateSceneProxy()
{
	return FSkinnedMeshSceneProxyDesc::CreateSceneProxy(SceneProxyDesc, bHideSkin);
}

void FFastGeoSkinnedMeshComponent::ApplyWorldTransform(const FTransform& InTransform)
{
	Super::ApplyWorldTransform(InTransform);
	WorldBounds = LocalBounds.TransformBy(WorldTransform);
}

