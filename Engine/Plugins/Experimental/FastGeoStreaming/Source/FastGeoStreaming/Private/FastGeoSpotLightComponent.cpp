// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoSpotLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "SpotLightSceneProxy.h"
#include "SpotLightSceneProxyDesc.h"

const FFastGeoElementType FFastGeoSpotLightComponent::Type(&FFastGeoPointLightComponentBase::Type);

FFastGeoSpotLightComponent::FFastGeoSpotLightComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

#if WITH_EDITOR
void FFastGeoSpotLightComponent::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	USpotLightComponent* SpotLightComponent = CastChecked<USpotLightComponent>(Component);
	SceneProxyDesc = USpotLightComponent::BuildSceneProxyDesc(*SpotLightComponent);
	SceneProxyDesc.LightComponent = nullptr;
}
#endif

FLightSceneProxy* FFastGeoSpotLightComponent::CreateTypedSceneProxy()
{
	check(SceneProxyDesc.SceneInterface == GetScene());
	return new FSpotLightSceneProxy(SceneProxyDesc);
}