// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoStaticMeshComponent.h"

#include "Engine/Engine.h"
#include "FastGeoContainer.h"
#include "FastGeoLog.h"
#include "FastGeoWeakElement.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Materials/MaterialRelevance.h"
#include "SceneInterface.h"
#include "MeshComponentHelper.h"
#include "StaticMeshComponentHelper.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/BodySetup.h"
#include "NaniteVertexFactory.h"
#include "StaticMeshSceneProxy.h"
#include "VT/MeshPaintVirtualTexture.h"

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#include "WorldPartition/HLOD/HLODActor.h"
#endif

const FFastGeoElementType FFastGeoStaticMeshComponentBase::Type(&FFastGeoMeshComponent::Type);

FFastGeoStaticMeshComponentBase::FFastGeoStaticMeshComponentBase(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

void FFastGeoStaticMeshComponentBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize persistent data from FStaticMeshSceneProxyDesc
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	Ar << SceneProxyDesc.StaticMesh;
	Ar << SceneProxyDesc.MeshPaintTexture;
	Ar << SceneProxyDesc.OverlayMaterial;
	Ar << SceneProxyDesc.MaterialSlotsOverlayMaterial;
	Ar << SceneProxyDesc.OverlayMaterialMaxDrawDistance;
	Ar << SceneProxyDesc.ForcedLodModel;
	Ar << SceneProxyDesc.MinLOD;
	Ar << SceneProxyDesc.WorldPositionOffsetDisableDistance;
	Ar << SceneProxyDesc.NanitePixelProgrammableDistance;
	Ar << SceneProxyDesc.DistanceFieldSelfShadowBias;
	Ar << SceneProxyDesc.DistanceFieldIndirectShadowMinVisibility;
	Ar << SceneProxyDesc.StaticLightMapResolution;
	Ar << SceneProxyDesc.MeshPaintTextureCoordinateIndex;
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bReverseCulling);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bEvaluateWorldPositionOffset);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bOverrideMinLOD);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bCastDistanceFieldIndirectShadow);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bOverrideDistanceFieldSelfShadowBias);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bEvaluateWorldPositionOffsetInRayTracing);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bSortTriangles);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bDisallowNanite);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bForceDisableNanite);
	FArchive_Serialize_BitfieldBool(Ar, SceneProxyDesc.bForceNaniteForMasked);
	Ar << bUseDefaultCollision;
}

void FFastGeoStaticMeshComponentBase::InitializeDynamicProperties()
{
#if !WITH_EDITOR
	// When using default collision, use the same collision profile as the StaticMesh
	if (bUseDefaultCollision)
	{
		if (UBodySetup* BodySetup = GetBodySetup())
		{
			BodyInstance.UseExternalCollisionProfile(BodySetup);
		}
	}
#endif

#if WITH_EDITOR
	check(GetWorld());
	if (GetWorld()->IsGameWorld())
#endif
	{
		FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
		if (UMeshPaintVirtualTexture* MeshPaintTexture = Cast<UMeshPaintVirtualTexture>(SceneProxyDesc.MeshPaintTexture))
		{
			// Cache the resource GT-side so the async render-state worker skips UTexture::GetResource.
			// May be null if texture PostLoad has not run yet; OnResourceUpdated below refreshes it.
			SceneProxyDesc.CachedMeshPaintTextureResource = MeshPaintTexture->GetResource();

			FWeakFastGeoComponent WeakComponent(this);
			MeshPaintTexture->GetOnResourceUpdated().Remove(OnResourceUpdated);
			OnResourceUpdated = MeshPaintTexture->GetOnResourceUpdated().AddLambda([WeakComponent]()
			{
				if (FFastGeoStaticMeshComponentBase* Component = WeakComponent.Get<FFastGeoStaticMeshComponentBase>())
				{
					FStaticMeshSceneProxyDesc& Desc = Component->GetStaticMeshSceneProxyDesc();
					if (UMeshPaintVirtualTexture* Texture = Cast<UMeshPaintVirtualTexture>(Desc.MeshPaintTexture))
					{
						Desc.CachedMeshPaintTextureResource = Texture->GetResource();
					}
					Component->MarkRenderStateDirty();
				}
			});

			// Attach to container unregistration
			UFastGeoContainer* Container = GetOwnerContainer();
			Container->GetOnUnregistered().Remove(OnContainerUnregistered);
			OnContainerUnregistered = Container->GetOnUnregistered().AddLambda([WeakComponent]()
			{
				if (FFastGeoStaticMeshComponentBase* Component = WeakComponent.Get<FFastGeoStaticMeshComponentBase>())
				{
					if (Component->OnResourceUpdated.IsValid())
					{
						UMeshPaintVirtualTexture* MeshPaintTexture = Cast<UMeshPaintVirtualTexture>(Component->GetStaticMeshSceneProxyDesc().MeshPaintTexture);
						if (ensure(MeshPaintTexture))
						{
							MeshPaintTexture->GetOnResourceUpdated().Remove(Component->OnResourceUpdated);
						}
						Component->OnResourceUpdated.Reset();
					}

					// Remove the component from the container unregistration event
					Component->GetOwnerContainer()->GetOnUnregistered().Remove(Component->OnContainerUnregistered);
					Component->OnContainerUnregistered.Reset();
				}
			});
		}
	}

	Super::InitializeDynamicProperties();
}

const Nanite::FResources* FFastGeoStaticMeshComponentBase::GetNaniteResources() const
{
	if (GetStaticMesh() && GetStaticMesh()->GetRenderData())
	{
		return GetStaticMesh()->GetRenderData()->NaniteResourcesPtr.Get();
	}
	return nullptr;
}

UBodySetup* FFastGeoStaticMeshComponentBase::GetBodySetup() const
{
	if (const FBodyInstancePhysicsCreationInputs* Cached = BodyInstance.GetAsyncPhysicsCreationInputs())
	{
		return Cached->ResolvedBodySetup;
	}

	return GetStaticMesh() ? GetStaticMesh()->GetBodySetup() : nullptr;
}

#if WITH_EDITOR
void FFastGeoStaticMeshComponentBase::NotifyRenderStateChanged()
{
	Super::NotifyRenderStateChanged();

	FObjectCacheEventSink::NotifyStaticMeshChanged_Concurrent(this);
}
#endif

bool FFastGeoStaticMeshComponentBase::IsNavigationRelevant() const
{
	return FStaticMeshComponentHelper::IsNavigationRelevant(*this);
}

FBox FFastGeoStaticMeshComponentBase::GetNavigationBounds() const
{
	return FStaticMeshComponentHelper::GetNavigationBounds(*this);
}

void FFastGeoStaticMeshComponentBase::GetNavigationData(FNavigationRelevantData& Data) const
{
	FStaticMeshComponentHelper::GetNavigationData(*this, Data);
}

bool FFastGeoStaticMeshComponentBase::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	return FStaticMeshComponentHelper::DoCustomNavigableGeometryExport(*this, GeomExport);
}

bool FFastGeoStaticMeshComponentBase::ShouldExportAsObstacle(const UNavCollisionBase& InNavCollision) const
{
	return InNavCollision.IsDynamicObstacle();
}

int32 FFastGeoStaticMeshComponentBase::GetNumMaterials() const
{
	return GetStaticMesh() ? GetStaticMesh()->GetStaticMaterials().Num() : 0;
}

UMaterialInterface* FFastGeoStaticMeshComponentBase::GetMaterial(int32 MaterialIndex) const
{
	return GetMaterial(MaterialIndex, false);
}

UMaterialInterface* FFastGeoStaticMeshComponentBase::GetNaniteAuditMaterial(int32 MaterialIndex) const
{
	return GetMaterial(MaterialIndex, true);
}

void FFastGeoStaticMeshComponentBase::ForEachMaterial(TFunctionRef<void(UMaterialInterface*, bool bIsNaniteOverride)> Func) const
{
	auto AddMaterial = [&Func](UMaterialInterface* Mat)
	{
		Func(Mat, false);
		if (UMaterialInterface* NaniteOverride = Mat->GetNaniteOverride())
		{
			Func(NaniteOverride, true);
		}
	};

	for (const TObjectPtr<UMaterialInterface>& Mat : OverrideMaterials)
	{
		if (Mat)
		{
			AddMaterial(Mat);
		}
	}

	if (UStaticMesh* Mesh = GetStaticMesh())
	{
		for (const FStaticMaterial& SlotMat : Mesh->GetStaticMaterials())
		{
			if (SlotMat.MaterialInterface)
			{
				AddMaterial(SlotMat.MaterialInterface);
			}
			if (SlotMat.OverlayMaterialInterface)
			{
				AddMaterial(SlotMat.OverlayMaterialInterface);
			}
		}
	}

	// Component overlay material
	const FStaticMeshSceneProxyDesc& Desc = GetStaticMeshSceneProxyDesc();
	if (Desc.OverlayMaterial)
	{
		AddMaterial(Desc.OverlayMaterial);
	}

	// Component per-slot overlay materials
	for (const TObjectPtr<UMaterialInterface>& Mat : Desc.MaterialSlotsOverlayMaterial)
	{
		if (Mat)
		{
			AddMaterial(Mat);
		}
	}
}

bool FFastGeoStaticMeshComponentBase::UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const
{
	return Nanite::FNaniteResourcesHelper::UseNaniteOverrideMaterials(*this, bDoingMaterialAudit);
}

bool FFastGeoStaticMeshComponentBase::ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials) const
{
	return Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(GetStaticMeshSceneProxyDesc(), OutNaniteMaterials);
}

UMaterialInterface* FFastGeoStaticMeshComponentBase::GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit) const
{
	return FStaticMeshComponentHelper::GetMaterial(*this, MaterialIndex, bDoingNaniteMaterialAudit);
}

const FCollisionResponseContainer& FFastGeoStaticMeshComponentBase::GetCollisionResponseToChannels() const
{
	return BodyInstance.GetResponseToChannels();
}

bool FFastGeoStaticMeshComponentBase::HasValidNaniteData() const
{
	return Nanite::FNaniteResourcesHelper::HasValidNaniteData(*this);
}

#if WITH_EDITOR
void FFastGeoStaticMeshComponentBase::InitializeSceneProxyDescFromComponent(UActorComponent* Component)
{
	UStaticMeshComponent* StaticMeshComponent = CastChecked<UStaticMeshComponent>(Component);
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	SceneProxyDesc.InitializeFromStaticMeshComponent(StaticMeshComponent);

	SceneProxyDesc.MeshPaintTexture = nullptr;
	SceneProxyDesc.MeshPaintTextureCoordinateIndex = 0;
	if (UTexture* Texture = StaticMeshComponent->GetMeshPaintTexture())
	{
		FImage Image;
		if (Texture->Source.GetMipImage(Image, 0))
		{
			UMeshPaintVirtualTexture* NewTexture = NewObject<UMeshPaintVirtualTexture>(GetOwnerContainer());
			NewTexture->Source.Init(Image);
			NewTexture->UpdateResource();

			SceneProxyDesc.MeshPaintTexture = NewTexture;
			SceneProxyDesc.MeshPaintTextureCoordinateIndex = StaticMeshComponent->GetMeshPaintTextureCoordinateIndex();
		}
	}
}

void FFastGeoStaticMeshComponentBase::InitializeFromComponent(UActorComponent* Component)
{
	Super::InitializeFromComponent(Component);

	UStaticMeshComponent* StaticMeshComponent = CastChecked<UStaticMeshComponent>(Component);
	bUseDefaultCollision = StaticMeshComponent->bUseDefaultCollision;
	LocalBounds = GetStaticMesh()->GetBounds();
	WorldBounds = LocalBounds.TransformBy(WorldTransform);
}

void FFastGeoStaticMeshComponentBase::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	// Unsupported properties
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	SceneProxyDesc.LODData = TArrayView<struct FStaticMeshComponentLODInfo>();
	SceneProxyDesc.LODParentPrimitive = nullptr;
#if STATICMESH_ENABLE_DEBUG_RENDERING
	SceneProxyDesc.bDrawMeshCollisionIfComplex = false;
	SceneProxyDesc.bDrawMeshCollisionIfSimple = false;
#endif
	SceneProxyDesc.bDisplayNaniteFallbackMesh = false;
	SceneProxyDesc.SectionIndexPreview = INDEX_NONE;
	SceneProxyDesc.MaterialIndexPreview = INDEX_NONE;
	SceneProxyDesc.SelectedEditorMaterial = INDEX_NONE;
	SceneProxyDesc.SelectedEditorSection = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	SceneProxyDesc.MaterialStreamingRelativeBoxes = TArrayView<uint32>();
#endif

	// Properties that will be initialized by InitializeSceneProxyDescDynamicProperties
	SceneProxyDesc.NaniteResources = nullptr;
	SceneProxyDesc.BodySetup = nullptr;
	SceneProxyDesc.MaterialRelevance = FMaterialRelevance();
	SceneProxyDesc.bUseProvidedMaterialRelevance = false;
}
#endif

void FFastGeoStaticMeshComponentBase::ApplyWorldTransform(const FTransform& InTransform)
{
	Super::ApplyWorldTransform(InTransform);

	WorldBounds = LocalBounds.TransformBy(WorldTransform);
}

void FFastGeoStaticMeshComponentBase::InitializeSceneProxyDescDynamicProperties()
{
	Super::InitializeSceneProxyDescDynamicProperties();

	// Initialize non-serialized properties
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	SceneProxyDesc.OverrideMaterials = OverrideMaterials;
	SceneProxyDesc.NaniteResources = GetNaniteResources();
	SceneProxyDesc.BodySetup = GetBodySetup();
	SceneProxyDesc.SetMaterialRelevance(GetMaterialRelevance(GetScene()->GetShaderPlatform()));
	SceneProxyDesc.SetCollisionResponseToChannels(GetCollisionResponseToChannels());

	// Add LODData support
	// Add LODParentPrimitive support
}

FPrimitiveSceneProxy* FFastGeoStaticMeshComponentBase::CreateSceneProxy(ESceneProxyCreationError* OutError)
{
	check(GetWorld());
	FSceneInterface* Scene = GetScene();
	UStaticMesh* StaticMesh = GetStaticMesh();
	check(Scene);
	check(StaticMesh);
	check(StaticMesh->GetRenderData());
	check(StaticMesh->GetRenderData()->IsInitialized());
	// IsCompiling() reads UStaticMesh state mutated on GT during async compile workflows. This
	// read is safe by configuration gating: cooked builds compile-out async compile entirely
	// (!WITH_EDITOR), and Editor builds disable IsAsyncRenderWorkAllowed so workers run under
	// FTaskTagScope(EParallelGameThread) with GT blocked. Flipping either gate would re-open
	// a race here.
	check(!StaticMesh->IsCompiling());

	if (OutError)
	{
		*OutError = ESceneProxyCreationError::None;
	}

	InitializeSceneProxyDescDynamicProperties();

	const FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	check(SceneProxyDesc.Scene);
	check(SceneProxyDesc.Scene == Scene);
	check(SceneProxyDesc.World == Scene->GetWorld());
	check(SceneProxyDesc.FeatureLevel == Scene->GetFeatureLevel());
	check(SceneProxyDesc.ComponentId == GetPrimitiveSceneId());

	FStaticMeshComponentHelper::ESceneProxyCreationError SceneProxyCreationError;
	FPrimitiveSceneProxy* SceneProxy = FStaticMeshComponentHelper::CreateSceneProxy<FFastGeoStaticMeshComponentBase, /*bRenderDataReady=*/true>(*this, &SceneProxyCreationError);

	if (SceneProxy == nullptr && OutError)
	{
		switch (SceneProxyCreationError)
		{
		case FStaticMeshComponentHelper::ESceneProxyCreationError::WaitingPSOs:
			*OutError = ESceneProxyCreationError::WaitingPSOs;
			break;

		default:
			*OutError = ESceneProxyCreationError::InvalidMesh;
			break;
		}
	}

	return SceneProxy;
}

FPrimitiveSceneProxy* FFastGeoStaticMeshComponentBase::CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	check(GetWorld());
	const FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	check(SceneProxyDesc.Scene);

	if (bCreateNanite)
	{
		PrimitiveSceneData.SceneProxy = ::new Nanite::FSceneProxy(NaniteMaterials, SceneProxyDesc);
	}
	else
	{
		PrimitiveSceneData.SceneProxy = ::new FStaticMeshSceneProxy(SceneProxyDesc, false);
	}
	return PrimitiveSceneData.SceneProxy;
}

void FFastGeoStaticMeshComponentBase::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	FStaticMeshComponentHelper::GetUsedMaterials(*this, OutMaterials, bGetDebugMaterials);
}

void FFastGeoStaticMeshComponentBase::GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	//We add null entry for every section of every LOD, this is a requirement for the MeshComponent this class derived from
	OutMaterialSlotOverlayMaterials.Reset();
	if (GetStaticMesh())
	{
		const TArray<FStaticMaterial>& StaticMaterials = GetStaticMesh()->GetStaticMaterials();
		for (const FStaticMaterial& StaticMaterial : StaticMaterials)
		{
			OutMaterialSlotOverlayMaterials.Add(StaticMaterial.OverlayMaterialInterface);
		}
	}
}

const TArray<TObjectPtr<UMaterialInterface>>& FFastGeoStaticMeshComponentBase::GetComponentMaterialSlotsOverlayMaterial() const
{
	return GetStaticMeshSceneProxyDesc().MaterialSlotsOverlayMaterial;
}

UStaticMesh* FFastGeoStaticMeshComponentBase::GetStaticMesh() const
{
	return GetStaticMeshSceneProxyDesc().StaticMesh;
}

IPrimitiveComponent* FFastGeoStaticMeshComponentBase::GetPrimitiveComponentInterface()
{
	return this;
}

UObject const* FFastGeoStaticMeshComponentBase::AdditionalStatObject() const
{
	return GetStaticMesh();
}

UMaterialInterface* FFastGeoStaticMeshComponentBase::GetOverlayMaterial() const
{
	return GetStaticMeshSceneProxyDesc().OverlayMaterial;
}

bool FFastGeoStaticMeshComponentBase::IsReverseCulling() const
{
	return GetStaticMeshSceneProxyDesc().IsReverseCulling();
}

bool FFastGeoStaticMeshComponentBase::IsDisallowNanite() const
{
	return GetStaticMeshSceneProxyDesc().IsDisallowNanite();
}

bool FFastGeoStaticMeshComponentBase::IsForceDisableNanite() const
{
	return GetStaticMeshSceneProxyDesc().IsForceDisableNanite();
}

bool FFastGeoStaticMeshComponentBase::IsForceNaniteForMasked() const
{
	return GetStaticMeshSceneProxyDesc().IsForceNaniteForMasked();
}

int32 FFastGeoStaticMeshComponentBase::GetForcedLodModel() const
{
	return GetStaticMeshSceneProxyDesc().GetForcedLodModel();
}

bool FFastGeoStaticMeshComponentBase::GetOverrideMinLOD() const
{
	return GetStaticMeshSceneProxyDesc().bOverrideMinLOD;
}

int32 FFastGeoStaticMeshComponentBase::GetMinLOD() const
{
	return GetStaticMeshSceneProxyDesc().MinLOD;
}

bool FFastGeoStaticMeshComponentBase::GetForceNaniteForMasked() const
{
	return GetStaticMeshSceneProxyDesc().bForceNaniteForMasked;
}

int32 FFastGeoStaticMeshComponentBase::GetWorldPositionOffsetDisableDistance() const
{
	return GetStaticMeshSceneProxyDesc().GetWorldPositionOffsetDisableDistance();
}

float FFastGeoStaticMeshComponentBase::GetNanitePixelProgrammableDistance() const
{
	return GetStaticMeshSceneProxyDesc().GetNanitePixelProgrammableDistance();
}

bool FFastGeoStaticMeshComponentBase::EvaluateWorldPositionOffsetInRayTracing() const
{
	return GetStaticMeshSceneProxyDesc().EvaluateWorldPositionOffsetInRayTracing();
}

#if WITH_EDITORONLY_DATA
bool FFastGeoStaticMeshComponentBase::IsDisplayNaniteFallbackMesh() const
{
	return GetStaticMeshSceneProxyDesc().IsDisplayNaniteFallbackMesh();
}
#endif

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FFastGeoStaticMeshComponentBase::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	FStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}
#endif

const FFastGeoElementType FFastGeoStaticMeshComponent::Type(&FFastGeoStaticMeshComponentBase::Type);

FFastGeoStaticMeshComponent::FFastGeoStaticMeshComponent(int32 InComponentIndex, FFastGeoElementType InType)
	: Super(InComponentIndex, InType)
{
}

void FFastGeoStaticMeshComponent::GetBodyInstances(TArray<FBodyInstance*>& OutBodyInstances)
{
	OutBodyInstances.Add(&BodyInstance);
}