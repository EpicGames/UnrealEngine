// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoStreamingModule.h"
#include "FastGeoPrimitiveComponent.h"
#include "FastGeoStaticMeshComponent.h"
#include "FastGeoInstancedStaticMeshComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "Engine/World.h"

IMPLEMENT_MODULE(FFastGeoStreamingModule, FastGeoStreaming);

bool FFastGeoStreamingModule::bIsFastGeoEnabled = true;
FAutoConsoleVariableRef FFastGeoStreamingModule::CVarIsEnabled(
	TEXT("FastGeo.Enable"),
	FFastGeoStreamingModule::bIsFastGeoEnabled,
	TEXT("Set to false to disable FastGeo (used in PIE and at cook time)."),
	ECVF_Default | ECVF_ReadOnly);

#if !WITH_EDITOR
bool FFastGeoStreamingModule::bAllowAsyncRenderWork = true;
FAutoConsoleVariableRef FFastGeoStreamingModule::CVarAllowAsyncRenderWork(
	TEXT("FastGeo.AllowAsyncRenderWork"),
	FFastGeoStreamingModule::bAllowAsyncRenderWork,
	TEXT("Cooked-only. When true, routes render-state Create/Destroy/PSO precache through the async UE::Tasks pipe (worker threads). When false, runs the same work synchronously in per-frame time-sliced ParallelFor passes on the game thread. Has no effect in editor or PIE - those always run synchronous time-sliced."),
	ECVF_ReadOnly);
#endif

bool FFastGeoStreamingModule::IsAsyncRenderWorkAllowed()
{
#if WITH_EDITOR
	return false;
#else
	return bAllowAsyncRenderWork;
#endif
}

void FFastGeoStreamingModule::StartupModule()
{
	IPrimitiveComponent::AddImplementer(TComponentInterfaceImplementation<IPrimitiveComponent>(UFastGeoContainer::StaticClass(),  [](UObject* Obj)
	{
		return CastChecked<UFastGeoContainer>(Obj)->GetPrimitiveComponents();
	}));

	IStaticMeshComponent::AddImplementer(TComponentInterfaceImplementation<IStaticMeshComponent>(UFastGeoContainer::StaticClass(),  [](UObject* Obj)
	{
		return CastChecked<UFastGeoContainer>(Obj)->GetStaticMeshComponents();
	}));

	Handle_OnWorldPreSendAllEndOfFrameUpdates = FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.AddRaw(this, &FFastGeoStreamingModule::OnWorldPreSendAllEndOfFrameUpdates);
}

void FFastGeoStreamingModule::ShutdownModule()
{
	IPrimitiveComponent::RemoveImplementer(UFastGeoContainer::StaticClass());
	IStaticMeshComponent::RemoveImplementer(UFastGeoContainer::StaticClass());

	FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.Remove(Handle_OnWorldPreSendAllEndOfFrameUpdates);
}

bool FFastGeoStreamingModule::IsFastGeoEnabled()
{
	return bIsFastGeoEnabled;
}

void FFastGeoStreamingModule::OnWorldPreSendAllEndOfFrameUpdates(UWorld* InWorld)
{
	if (UFastGeoWorldSubsystem* WorldSubsystem = InWorld->GetSubsystem<UFastGeoWorldSubsystem>())
	{
		WorldSubsystem->ProcessPendingRecreate();
	}
}
