// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoSurrogateActor.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoSurrogateComponentDescriptor.h"
#include "FastGeoContainer.h"
#include "Engine/Level.h"
#include "Engine/World.h"

AFastGeoSurrogateActor::AFastGeoSurrogateActor()
{
	PrimaryActorTick.bCanEverTick = false;
	bReplicates = false;
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("FastGeoSurrogateRootComponent"));
	RootComponent->SetMobility(EComponentMobility::Static);
	SetRootComponent(RootComponent);
}

void AFastGeoSurrogateActor::PostInitializeComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AFastGeoSurrogateActor::PostInitializeComponents);
	Super::PostInitializeComponents();

	if (!UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents())
	{
		return;
	}

	bIsActive = true;

	// Prepare SurrogateComponents
	TArray<UFastGeoSurrogateComponent*> Components;
	GetComponents<UFastGeoSurrogateComponent>(Components);
	SurrogateComponents.SetNumZeroed(Components.Num());
	for (UFastGeoSurrogateComponent* Component : Components)
	{
		// Descriptor count is equal to surrogate component count, so descriptor index should always be valid
		if (ensure(SurrogateComponents.IsValidIndex(Component->GetDescriptorIndex())))
		{
			SurrogateComponents[Component->GetDescriptorIndex()] = Component;
		}
	}

	// Register FastGeo content
	check(!bHasRegisteredWithFastGeo);
	if (!bHasRegisteredWithFastGeo && GetWorld() && GetWorld()->IsGameWorld())
	{
		ULevel* Level = GetLevel();
		if (UFastGeoContainer* FastGeo = Level ? Level->GetAssetUserData<UFastGeoContainer>() : nullptr; ensure(FastGeo))
		{
			FastGeo->Register();
			bHasRegisteredWithFastGeo = true;
		}
	}
}

void AFastGeoSurrogateActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	UnregisterFromFastGeo();
}

void AFastGeoSurrogateActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Normally, PostUnregisterAllComponents should have been called, but NOT in the case 
	// where RemoveFromWorld was called with bAllowIncrementalRemoval = false.
	// In this case, we need to wait for FastGeo to complete.
	if (UnregisterFromFastGeo())
	{
		ULevel* Level = GetLevel();
		if (UFastGeoContainer* FastGeo = Level ? Level->GetAssetUserData<UFastGeoContainer>() : nullptr; ensure(FastGeo))
		{
			const bool bWaitForCompletion = true;
			FastGeo->Tick(bWaitForCompletion);
		}
	}

	Super::EndPlay(EndPlayReason);
}

bool AFastGeoSurrogateActor::UnregisterFromFastGeo()
{
	if (!UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AFastGeoSurrogateActor::UnregisterFromFastGeo);
	bIsActive = false;

	if (bHasRegisteredWithFastGeo)
	{
		ULevel* Level = GetLevel();
		if (UFastGeoContainer* FastGeo = Level ? Level->GetAssetUserData<UFastGeoContainer>() : nullptr; ensure(FastGeo))
		{
			check(FastGeo->GetSurrogateActor() == this);

			// Mutate body instances to make sure any direct access to the bodyinstance will return the intended value.
			// Direct BodyInstance access is safe because it only changes local CPU data, no physics-thread sync is required.
			ForEachComponent<UFastGeoSurrogateComponent>(false, [](UFastGeoSurrogateComponent* SurrogateComponent)
			{
				SurrogateComponent->GetBodyInstance()->SetCollisionEnabled(ECollisionEnabled::NoCollision, false);
				SurrogateComponent->GetBodyInstance()->SetResponseToAllChannels(ECR_Ignore);
			});

			FastGeo->Unregister();
			bHasRegisteredWithFastGeo = false;
			return true;
		}
	}
	return false;
}

UFastGeoSurrogateComponent* AFastGeoSurrogateActor::GetSurrogateComponent(int32 SurrogateComponentDescriptorIndex) const
{
	if (!UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents())
	{ 
		return nullptr;
	}
	if (ensure(SurrogateComponents.IsValidIndex(SurrogateComponentDescriptorIndex)))
	{
		return SurrogateComponents[SurrogateComponentDescriptorIndex];
	}
	return nullptr;
}

const FFastGeoSurrogateComponentDescriptor* AFastGeoSurrogateActor::GetSurrogateComponentDescriptor(int32 SurrogateComponentDescriptorIndex) const
{
	if (ensure(SurrogateComponentDescriptors.IsValidIndex(SurrogateComponentDescriptorIndex)))
	{
		return &SurrogateComponentDescriptors[SurrogateComponentDescriptorIndex];
	}
	return nullptr;
}