// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoSurrogateComponent.h"
#include "FastGeoSurrogateActor.h"
#include "FastGeoContainer.h"
#include "FastGeoComponent.h"
#include "FastGeoPrimitiveComponent.h"
#include "FastGeoWorldSubsystem.h"
#include "FastGeoSurrogateBodyInstanceIndex.h"
#include "FastGeoSurrogateComponentDescriptor.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"

UFastGeoSurrogateComponent::UFastGeoSurrogateComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
	SetHiddenInGame(true);
	SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
	SetCollisionObjectType(ECC_WorldStatic);
	SetCollisionResponseToAllChannels(ECR_Block);
	SetGenerateOverlapEvents(false);
	SetCanEverAffectNavigation(false);
	bAffectDynamicIndirectLighting = false;
	bAffectDistanceFieldLighting = false;
	bNeverNeedsRenderUpdate = true;
	SetMobility(EComponentMobility::Static);
}

void UFastGeoSurrogateComponent::OnRegister()
{
	if (AFastGeoSurrogateActor* SurrogateActor = Cast<AFastGeoSurrogateActor>(GetOwner()); ensure(SurrogateActor))
	{
		if (const FFastGeoSurrogateComponentDescriptor* ComponentDesc = SurrogateActor->GetSurrogateComponentDescriptor(DescriptorIndex); ensure(ComponentDesc))
		{
			SetSurrogateComponentDescriptor(*ComponentDesc);
		}
	}

	Super::OnRegister();
}

void UFastGeoSurrogateComponent::SetSurrogateComponentDescriptor(const FFastGeoSurrogateComponentDescriptor& InComponentDesc)
{
	check(!IsPhysicsStateCreated());
	BodyInstance.SetCollisionEnabled(InComponentDesc.CollisionEnabled);
	BodyInstance.SetObjectType(InComponentDesc.ObjectType);
	BodyInstance.SetResponseToChannels(InComponentDesc.ResponseContainer);
	BodyInstance.SetWalkableSlopeOverride(InComponentDesc.WalkableSlopeOverride, true);
	CanCharacterStepUpOn = InComponentDesc.CanCharacterStepUpOn;
}

FBoxSphereBounds UFastGeoSurrogateComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	AActor* Owner = GetOwner();
	ULevel* Level = Owner ? Owner->GetLevel() : nullptr;
	if (const IWorldPartitionCell* Cell = Level ? Level->GetWorldPartitionRuntimeCell() : nullptr)
	{
		return Cell->GetContentBounds();
	}

	const FVector Origin = LocalToWorld.GetLocation();
	return FBoxSphereBounds(Origin, FVector(1.f), 1.f);
}

bool UFastGeoSurrogateComponent::ShouldCreatePhysicsState() const
{
	if (!UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents())
	{
		return false;
	}

	if (!HasRegisteredBodyInstances())
	{
		return false;
	}

	return Super::ShouldCreatePhysicsState();
}

void UFastGeoSurrogateComponent::OnCreatePhysicsState()
{
	// Skip UPrimitiveComponent implementation - the surrogate does not create its own physics bodies
	USceneComponent::OnCreatePhysicsState();

	OnComponentPhysicsStateChanged.Broadcast(this, EComponentPhysicsStateChange::Created);
}

void UFastGeoSurrogateComponent::OnDestroyPhysicsState()
{
	// Skip UPrimitiveComponent implementation - the surrogate does not create its own physics bodies
	USceneComponent::OnDestroyPhysicsState();

	OnComponentPhysicsStateChanged.Broadcast(this, EComponentPhysicsStateChange::Destroyed);
}

ECollisionEnabled::Type UFastGeoSurrogateComponent::GetCollisionEnabled() const
{
	if (AFastGeoSurrogateActor* SurrogateOwner = Cast<AFastGeoSurrogateActor>(GetOwner()))
	{
		if (!SurrogateOwner->IsActive())
		{
			return ECollisionEnabled::NoCollision;
		}
	}
	return Super::GetCollisionEnabled();
}

ECollisionResponse UFastGeoSurrogateComponent::GetCollisionResponseToChannel(ECollisionChannel Channel) const
{
	if (AFastGeoSurrogateActor* SurrogateOwner = Cast<AFastGeoSurrogateActor>(GetOwner()))
	{
		if (!SurrogateOwner->IsActive())
		{
			return ECollisionResponse::ECR_Ignore;
		}
	}
	return Super::GetCollisionResponseToChannel(Channel);
}

const FCollisionResponseContainer& UFastGeoSurrogateComponent::GetCollisionResponseToChannels() const
{
	if (AFastGeoSurrogateActor* SurrogateOwner = Cast<AFastGeoSurrogateActor>(GetOwner()))
	{
		if (!SurrogateOwner->IsActive())
		{
			static const FCollisionResponseContainer AllIgnoreContainer = []() 
			{
				FCollisionResponseContainer Container;
				Container.SetAllChannels(ECollisionResponse::ECR_Ignore);
				return Container;
			}();

			return AllIgnoreContainer;
		}
	}
	return Super::GetCollisionResponseToChannels();
}

void UFastGeoSurrogateComponent::RegisterBodyInstances(const TArray<FBodyInstance*>& InBodyInstances)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoSurrogateComponent::RegisterBodyInstances);
	check(IndexToBodyInstance.IsEmpty());
	check(!IsPhysicsStateCreated());
	IndexToBodyInstance.SetNumZeroed(InBodyInstances.Num());
	for (FBodyInstance* BodyInst : InBodyInstances)
	{
		if (BodyInst)
		{
			check(FFastGeoSurrogateBodyInstanceIndex::IsEncoded(BodyInst->InstanceBodyIndex));
			const int32 ArrayIndex = FFastGeoSurrogateBodyInstanceIndex::Decode(BodyInst->InstanceBodyIndex);
			check(IndexToBodyInstance.IsValidIndex(ArrayIndex));
			check(!IndexToBodyInstance[ArrayIndex]);
			IndexToBodyInstance[ArrayIndex] = BodyInst;
		}
	}

	// Body instances are now accessible -- create physics state through the
	// normal engine lifecycle. ShouldCreatePhysicsState gates on
	// HasRegisteredBodyInstances, so this only succeeds after population.
	CreatePhysicsState();
}

void UFastGeoSurrogateComponent::UnregisterBodyInstances()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoSurrogateComponent::UnregisterBodyInstances);

	// Destroy physics state before clearing body instances so that
	// IsPhysicsStateCreated accurately reflects that the surrogate no
	// longer has valid body instances. Goes through the normal engine
	// lifecycle (delegates, OnDestroyPhysicsState, etc.).
	if (IsPhysicsStateCreated())
	{
		DestroyPhysicsState();
	}

	IndexToBodyInstance.Reset();
}

FBodyInstance* UFastGeoSurrogateComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	if (Index != INDEX_NONE && FFastGeoSurrogateBodyInstanceIndex::IsEncoded(Index))
	{
		const int32 LocalIndex = FFastGeoSurrogateBodyInstanceIndex::Decode(Index);
		if (IndexToBodyInstance.IsValidIndex(LocalIndex))
		{
			check(IndexToBodyInstance[LocalIndex]);
			return IndexToBodyInstance[LocalIndex];
		}
		return nullptr;
	}
	return Super::GetBodyInstance(BoneName, bGetWelded, Index);
}

Chaos::FPhysicsObject* UFastGeoSurrogateComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	// The Id is expected to be an encoded surrogate body index, sourced from HitResult.Item by the engine's collision conversion. Callers
	// that pass 0/INDEX_NONE for "the root body" can't be served by a multi-body surrogate, so we return nullptr in that case.
	if (!FFastGeoSurrogateBodyInstanceIndex::IsEncoded(Id))
	{
		return nullptr;
	}

	const int32 LocalIndex = FFastGeoSurrogateBodyInstanceIndex::Decode(Id);
	if (!IndexToBodyInstance.IsValidIndex(LocalIndex))
	{
		return nullptr;
	}

	const FBodyInstance* BI = IndexToBodyInstance[LocalIndex];
	if (!BI || !BI->IsValidBodyInstance())
	{
		return nullptr;
	}

	return BI->GetPhysicsActor()->GetPhysicsObject();
}

TArray<Chaos::FPhysicsObject*> UFastGeoSurrogateComponent::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObject*> Result;
	Result.Reserve(IndexToBodyInstance.Num());
	for (const FBodyInstance* BI : IndexToBodyInstance)
	{
		if (BI && BI->IsValidBodyInstance())
		{
			Result.Add(BI->GetPhysicsActor()->GetPhysicsObject());
		}
	}
	return Result;
}