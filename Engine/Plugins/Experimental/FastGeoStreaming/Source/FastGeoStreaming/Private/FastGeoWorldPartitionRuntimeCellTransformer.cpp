// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoWorldPartitionRuntimeCellTransformer.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "FastGeoStreamingModule.h"

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"
#include "Engine/Blueprint.h"
#include "UObject/Package.h"
#include "Modules/ModuleManager.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Streaming/ActorTextureStreamingBuildDataComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/DecalComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/RectLightComponent.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/Info.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionRuntimeCellTransformerISM.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceEditorPivotInterface.h"
#include "ActorEditorUtils.h"
#include "Animation/AnimBank.h"
#include "Animation/AnimInstance.h"
#include "Selection.h"
#include "FastGeoContainer.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoComponent.h"
#include "FastGeoHLOD.h"
#include "FastGeoStaticMeshComponent.h"
#include "FastGeoInstancedStaticMeshComponent.h"
#include "FastGeoSkinnedMeshComponent.h"
#include "FastGeoInstancedSkinnedMeshComponent.h"
#include "FastGeoSurrogateActor.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoSurrogateComponentDescriptor.h"
#include "FastGeoSurrogateBodyInstanceIndex.h"
#include "FastGeoLog.h"
#endif

#if WITH_EDITORONLY_DATA
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/CustomHLODActor.h"
#endif

#define LOCTEXT_NAMESPACE "WorldPartition"

#if WITH_EDITOR
namespace FastGeo
{
	// Used to cancel package being dirtied when bDebugMode is modified (see PostEditChangeProperty)
	static bool GPackageWasDirty = false;

	// Tag use to force include actors into FastGeoStreaming
	static const FName NAME_FastGeo(TEXT("FastGeo"));

	// Tag use to force exclude actors from FastGeoStreaming
	static const FName NAME_NoFastGeo(TEXT("NoFastGeo"));

	static constexpr const TCHAR* LogSeparatorOuter = TEXT("========================================================================");
	static constexpr const TCHAR* LogSeparator = TEXT("------------------------------------------------------------------------");

	static void LogDebugSelectionMessage(const TCHAR* Message)
	{
		UE_LOGF(LogFastGeoStreaming, Log, "%ls", LogSeparatorOuter);
		UE_LOGF(LogFastGeoStreaming, Log, "FastGeo Debug Selection Mode: %ls", Message);
		UE_LOGF(LogFastGeoStreaming, Log, "%ls", LogSeparatorOuter);
	}

	static const IWorldPartitionHLODObject* GetHLODObject(const AActor* Actor)
	{
		if (const AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(Actor))
		{
			return HLODActor;
		}
		if (const AWorldPartitionCustomHLOD* CustomHLODActor = Cast<AWorldPartitionCustomHLOD>(Actor))
		{
			return CustomHLODActor;
		}
		return nullptr;
	}

	static bool IsHLODActor(const AActor* Actor)
	{
		return GetHLODObject(Actor) != nullptr;
	}

	static bool IsCollisionEnabled(const UPrimitiveComponent* Component)
	{
		return Component->IsCollisionEnabled() && !IsHLODActor(Component->GetOwner());
	}

	static FString GetComponentShortName(const UActorComponent* InComponent)
	{
		TStringBuilder<256> Builder;
		Builder += InComponent->GetOwner()->GetName();
		Builder += TEXT(".");
		Builder += InComponent->GetName();
		return *Builder;
	}

	struct FTransformationStats
	{
		int32 TotalActorCount = 0;
		int32 TotalComponentCount = 0;
		int32 FullyTransformableActorCount = 0;
		int32 PartiallyTransformableActorCount = 0;
		int32 TransformedComponentCount = 0;

		void DumpStats(const TCHAR* InPrefixString)
		{
			if (TotalActorCount)
			{
				const float FullyTransformedActorPercentage = TotalActorCount > 0 ? (100.f * FullyTransformableActorCount) / TotalActorCount : 0.f;
				const float PartiallyTransformedActorPercentage = TotalActorCount > 0 ? (100.f * PartiallyTransformableActorCount) / TotalActorCount : 0.f;
				const float TransformedComponentPercentage = TotalComponentCount > 0 ? (100.f * TransformedComponentCount) / TotalComponentCount : 0.f;
				const int32 NonTransformableActorCount = FMath::Max(0, TotalActorCount - FullyTransformableActorCount - PartiallyTransformableActorCount);
				const float NonTransformableActorPercentage = NonTransformableActorCount > 0 ? (100.f * NonTransformableActorCount) / TotalActorCount : 0.f;

				UE_CLOGF(FullyTransformableActorCount, LogFastGeoStreaming, Log, "%ls Transformable Actors (Full)    = %d (%3.1f%%)", InPrefixString, FullyTransformableActorCount, FullyTransformedActorPercentage);
				UE_CLOGF(PartiallyTransformableActorCount,LogFastGeoStreaming, Log, "%ls Transformable Actors (Partial) = %d (%3.1f%%)", InPrefixString, PartiallyTransformableActorCount, PartiallyTransformedActorPercentage);
				UE_CLOGF(TransformedComponentCount, LogFastGeoStreaming, Log, "%ls Transformable Components       = %d (%3.1f%%)", InPrefixString, TransformedComponentCount, TransformedComponentPercentage);
				UE_CLOGF(NonTransformableActorCount, LogFastGeoStreaming, Log, "%ls Non-Transformable Actors       = %d (%3.1f%%)", InPrefixString, NonTransformableActorCount, NonTransformableActorPercentage);
			}
		}
	};

	FFastGeoElementType GetFastGeoComponentType(TSubclassOf<USceneComponent> InClass)
	{
		static const TMap<TSubclassOf<USceneComponent>, FFastGeoElementType> FastGeoComponentTypeMapping =
		{
			{ UStaticMeshComponent::StaticClass(), FFastGeoStaticMeshComponent::Type },
			{ UInstancedStaticMeshComponent::StaticClass(), FFastGeoInstancedStaticMeshComponent::Type },
			{ USkinnedMeshComponent::StaticClass(), FFastGeoSkinnedMeshComponent::Type },
			{ UInstancedSkinnedMeshComponent::StaticClass(), FFastGeoInstancedSkinnedMeshComponent::Type },
			{ UDecalComponent::StaticClass(), FFastGeoDecalComponent::Type },
			{ UPointLightComponent::StaticClass(), FFastGeoPointLightComponent::Type },
			{ USpotLightComponent::StaticClass(), FFastGeoSpotLightComponent::Type },
			{ URectLightComponent::StaticClass(), FFastGeoRectLightComponent::Type },
		};

		// Walk the component class hierarchy and look for a fast geo mapping
		for (UClass* Class = InClass; Class; Class = Class->GetSuperClass())
		{
			if (const FFastGeoElementType* Found = FastGeoComponentTypeMapping.Find(Class))
			{
				return *Found;
			}
		}

		return FFastGeoElementType::Invalid;
	}
}

UFastGeoWorldPartitionRuntimeCellTransformer* UFastGeoWorldPartitionRuntimeCellTransformer::CurrentTransformer = nullptr;
#endif

void UFastGeoTransformerSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Fix old objects that were created when the transformer was missing RF_Transactional
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_Transactional))
	{
		SetFlags(RF_Transactional);
	}
#endif
}

UFastGeoWorldPartitionRuntimeCellTransformer::UFastGeoWorldPartitionRuntimeCellTransformer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	if (GIsEditor && !IsTemplate())
	{
		USelection::SelectionChangedEvent.AddUObject(this, &UFastGeoWorldPartitionRuntimeCellTransformer::OnSelectionChanged);
	}

	EmbeddedSettings = CreateDefaultSubobject<UFastGeoTransformerSettings>(TEXT("EmbeddedSettings"));
#endif
}

void UFastGeoWorldPartitionRuntimeCellTransformer::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		Ar << bDebugMode;
	}

	if (Ar.IsLoading())
	{
		// Convert setting properties to embedded settings asset
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FastGeoTransformerSettingAssets)
		{
			EmbeddedSettings->AllowedActorClasses                   = AllowedActorClasses_DEPRECATED;
			EmbeddedSettings->AllowedExactActorClasses              = AllowedExactActorClasses_DEPRECATED;
			EmbeddedSettings->AllowedComponentClasses               = AllowedComponentClasses_DEPRECATED;
			EmbeddedSettings->AllowedExactComponentClasses          = AllowedExactComponentClasses_DEPRECATED;
			EmbeddedSettings->DisallowedActorClasses                = DisallowedActorClasses_DEPRECATED;
			EmbeddedSettings->DisallowedExactActorClasses           = DisallowedExactActorClasses_DEPRECATED;
			EmbeddedSettings->DisallowedComponentClasses            = DisallowedComponentClasses_DEPRECATED;
			EmbeddedSettings->DisallowedExactComponentClasses       = DisallowedExactComponentClasses_DEPRECATED;
			EmbeddedSettings->IgnoredRemainingComponentClasses      = IgnoredRemainingComponentClasses_DEPRECATED;
			EmbeddedSettings->IgnoredRemainingExactComponentClasses = IgnoredRemainingExactComponentClasses_DEPRECATED;
		}
	}
#endif
}

#if WITH_EDITOR

bool UFastGeoWorldPartitionRuntimeCellTransformer::IsDebugModeEnabled = false;
FAutoConsoleVariableRef UFastGeoWorldPartitionRuntimeCellTransformer::CVarIsDebugModeEnabled(
	TEXT("FastGeo.Debug.Transformer"),
	UFastGeoWorldPartitionRuntimeCellTransformer::IsDebugModeEnabled,
	TEXT("When true, the FastGeo cell transformer logs a structured allow/reject/ignored report at cell transform time (used in PIE and at cook)."),
	ECVF_Default);

bool UFastGeoWorldPartitionRuntimeCellTransformer::IsFastGeoTransformerEnabled = true;
FAutoConsoleVariableRef UFastGeoWorldPartitionRuntimeCellTransformer::CVarIsFastGeoTransformerEnabled(
	TEXT("FastGeo.EnableTransformer"),
	UFastGeoWorldPartitionRuntimeCellTransformer::IsFastGeoTransformerEnabled,
	TEXT("Set to false to disable FastGeo cell transformer (used in PIE and at cook time). Has no effect if FastGeo.Enable is false."),
	ECVF_Default);

void UFastGeoWorldPartitionRuntimeCellTransformer::BeginDestroy()
{
	Super::BeginDestroy();

	if (GIsEditor && !IsTemplate())
	{
		USelection::SelectionChangedEvent.RemoveAll(this);
	}
}

bool UFastGeoWorldPartitionRuntimeCellTransformer::IsDebugMode() const
{
	return bDebugMode || UFastGeoWorldPartitionRuntimeCellTransformer::IsDebugModeEnabled;
}

void UFastGeoWorldPartitionRuntimeCellTransformer::OnSelectionChanged(UObject* Object)
{
	if (bDebugSelectionMode && IsEnabled())
	{
		if (!FFastGeoStreamingModule::IsFastGeoEnabled())
		{
			FastGeo::LogDebugSelectionMessage(TEXT("Skipped - FastGeo.Enable is false."));
			return;
		}

		if (!UFastGeoWorldPartitionRuntimeCellTransformer::IsFastGeoTransformerEnabled)
		{
			FastGeo::LogDebugSelectionMessage(TEXT("Skipped - FastGeo.EnableTransformer is false."));
			return;
		}

		TArray<AActor*> SelectedActors;
		TArray<AActor*> IgnoredActors;

		TFunction<void(AActor* InActor)> AddActorToSelection = [this, &SelectedActors, &IgnoredActors, &AddActorToSelection](AActor* InActor)
		{
			if (!CanAlwaysIgnoreActor(InActor))
			{
				const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor);
				if (LevelInstance && LevelInstance->GetDesiredRuntimeBehavior() == ELevelInstanceRuntimeBehavior::Partitioned && InActor->GetWorld())
				{
					if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = InActor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
					{
						LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [&AddActorToSelection](AActor* Actor)
						{
							AddActorToSelection(Actor);
							return true;
						});
					}
				}
				else
				{
					SelectedActors.Add(InActor);
				}
			}
			else if (IsValid(InActor))
			{
				IgnoredActors.Add(InActor);
			}
		};

		if (USelection* Selection = Cast<USelection>(Object))
		{
			for (int32 Index = 0; Index < Selection->Num(); Index++)
			{
				if (AActor* SelectedActor = Cast<AActor>(Selection->GetSelectedObject(Index)))
				{
					AddActorToSelection(SelectedActor);
				}
			}
		}

		if (!SelectedActors.IsEmpty() || !IgnoredActors.IsEmpty())
		{
			TGuardValue<bool> Guard(FFastGeoTransformResult::bShouldCollectReasons, true);

			FTransformationStats Stats;
			TArray<FActorReport> ActorReports;
			TArray<TPair<AActor*, FTransformableActor>> TransformableActors;

			if (!SelectedActors.IsEmpty())
			{
				GatherTransformableActors(SelectedActors, SelectedActors[0]->GetLevel(), TransformableActors, Stats, &ActorReports);
			}

			DumpDebugReport(TEXT("FastGeo Debug Selection Mode"), IgnoredActors, ActorReports, Stats);
		}
	}
}

bool UFastGeoWorldPartitionRuntimeCellTransformer::CanAlwaysIgnoreActor(AActor* InActor) const
{
	return InActor->IsA<AWorldSettings>() ||
		InActor->IsA<AWorldDataLayers>() ||
		InActor->IsA<ALevelInstanceEditorInstanceActor>() ||
		InActor->Implements<ULevelInstanceEditorPivotInterface>() ||
		FActorEditorUtils::IsABuilderBrush(InActor);
}

void UFastGeoWorldPartitionRuntimeCellTransformer::Transform(ULevel* InLevel)
{
	const bool bIsDebugMode = IsDebugMode();
	TGuardValue<bool> Guard(FFastGeoTransformResult::bShouldCollectReasons, bIsDebugMode);

	if (!FFastGeoStreamingModule::IsFastGeoEnabled() || !UFastGeoWorldPartitionRuntimeCellTransformer::IsFastGeoTransformerEnabled)
	{
		return;
	}

	if (!FPhysScene::SupportsAsyncPhysicsStateCreation() ||
		!FPhysScene::SupportsAsyncPhysicsStateDestruction())
	{
		UE_LOGF(LogFastGeoStreaming, Error, "FastGeoStreaming Cell Transformer requires 'p.Chaos.EnableAsyncInitBody' to be enabled.");
		return;
	}

	check(InLevel);
	check(!UFastGeoWorldPartitionRuntimeCellTransformer::CurrentTransformer);
	TGuardValue<UFastGeoWorldPartitionRuntimeCellTransformer*> GuardCurrentTransformer(UFastGeoWorldPartitionRuntimeCellTransformer::CurrentTransformer, this);

	FTransformationStats Stats;
	TArray<TPair<AActor*, FTransformableActor>> TransformableActors;
	TArray<FActorReport> ActorReports;
	GatherTransformableActors(InLevel->Actors, InLevel, TransformableActors, Stats, bIsDebugMode ? &ActorReports : nullptr);

	if (!TransformableActors.IsEmpty())
	{
		const FString CellName = InLevel->GetWorldPartitionRuntimeCell() ? Cast<UObject>(InLevel->GetWorldPartitionRuntimeCell())->GetName() : TEXT("Cell");
		UFastGeoContainer* FastGeo = NewObject<UFastGeoContainer>(InLevel, *FString::Printf(TEXT("FastGeoContainer_%s"), *CellName));
		InLevel->AddAssetUserData(FastGeo);

		TUniquePtr<FFastGeoComponentCluster> LevelComponentCluster(new FFastGeoComponentCluster(FastGeo, *FString::Printf(TEXT("FastGeoComponentCluster_%s"), *CellName)));

		TMap<int32, FFastGeoSurrogateBodyInstanceIndex> LevelSurrogateComponentBodyInstanceCounter;
		TArray<FFastGeoSurrogateComponentDescriptor> LevelSurrogateComponentDescriptors;

		for (const TPair<AActor*, FTransformableActor>& Entry : TransformableActors)
		{
			AActor* Actor = Entry.Key;

			FFastGeoComponentCluster* CurrentComponentCluster = LevelComponentCluster.Get();
			TUniquePtr<FFastGeoHLOD> FastGeoHLOD;
			const IWorldPartitionHLODObject* HLODActor = FastGeo::GetHLODObject(Actor);
			if (HLODActor)
			{
				FastGeoHLOD.Reset(new FFastGeoHLOD(FastGeo, *FString::Printf(TEXT("FastGeoHLOD_%s"), *Actor->GetName())));
				FastGeoHLOD->SetSourceCellGuid(HLODActor->GetSourceCellGuid());
				FastGeoHLOD->SetRequireWarmup(HLODActor->DoesRequireWarmup());

				if (HLODActor->IsStandalone())
				{
					FastGeoHLOD->SetStandaloneHLODGuid(HLODActor->GetStandaloneHLODGuid());
				}
				else if (HLODActor->IsCustomHLOD())
				{
					FastGeoHLOD->SetCustomHLODGuid(HLODActor->GetCustomHLODGuid());
				}

				CurrentComponentCluster = FastGeoHLOD.Get();
			}

			const TArray<UActorComponent*>& Components = Entry.Value.TransformableComponents;
			check(!Components.IsEmpty());
			for (UActorComponent* Component : Components)
			{
				FFastGeoElementType FastGeoComponentType = FastGeo::GetFastGeoComponentType(Component->GetClass());
				check(FastGeoComponentType.IsValid());

				FFastGeoComponent& FastGeoComponent = CurrentComponentCluster->AddComponent(FastGeoComponentType);
				FastGeoComponent.InitializeFromComponent(Component);

				if (GetSettings().bGenerateSurrogateComponents)
				{
					if (FFastGeoPrimitiveComponent* FastGeoPrimitiveComponent = FastGeoComponent.CastTo<FFastGeoPrimitiveComponent>(); FastGeoPrimitiveComponent && FastGeoPrimitiveComponent->IsCollisionEnabled())
					{
						UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Component);
						FFastGeoSurrogateComponentDescriptor ComponentDesc(*PrimitiveComponent);
						FastGeoPrimitiveComponent->SurrogateComponentDescriptorIndex = LevelSurrogateComponentDescriptors.AddUnique(ComponentDesc);
						FFastGeoSurrogateBodyInstanceIndex& NextAvailableInstanceBodyIndex = LevelSurrogateComponentBodyInstanceCounter.FindOrAdd(FastGeoPrimitiveComponent->SurrogateComponentDescriptorIndex, FFastGeoSurrogateBodyInstanceIndex::Encode(0));
						FastGeoPrimitiveComponent->ReserveSurrogateInstanceBodyIndices(NextAvailableInstanceBodyIndex);
					}
				}
			}

			// Once all components are transformed, remove them from the actor
			for (UActorComponent* Component : Components)
			{
				Actor->RemoveOwnedComponent(Component);
				Component->MarkAsGarbage();
			}

			const bool bIsActorFullyTransformed = Entry.Value.bIsActorFullyTransformable;
			if (bIsActorFullyTransformed)
			{
				InLevel->Actors[Entry.Value.ActorIndex] = nullptr;
			}
			else if (USceneComponent* OldRootComponent = Actor->GetRootComponent(); OldRootComponent && !IsValid(OldRootComponent))
			{
				// Replace removed root component with a SceneComponent and remap attachment of other components
				USceneComponent* NewRootComponent = NewObject<USceneComponent>(Actor);
				NewRootComponent->SetRelativeTransform(OldRootComponent->GetRelativeTransform());
				NewRootComponent->SetMobility(OldRootComponent->GetMobility());
				Actor->SetRootComponent(NewRootComponent);

				Actor->ForEachComponent<USceneComponent>(false, [OldRootComponent, NewRootComponent](USceneComponent* Component)
				{
					if (Component->GetAttachParent() == OldRootComponent)
					{
						Component->SetupAttachment(NewRootComponent);
					}
				});
			}

			if (FastGeoHLOD.IsValid())
			{
				check(FastGeoHLOD->HasComponents());
				FastGeo->AddComponentCluster(FastGeoHLOD.Get());
			}
		}

		// Add level component cluster (if not empty)
		if (LevelComponentCluster->HasComponents())
		{
			FastGeo->AddComponentCluster(LevelComponentCluster.Get());
		}

		if (LevelSurrogateComponentDescriptors.Num() > 0)
		{
			AFastGeoSurrogateActor* SurrogateActor = NewObject<AFastGeoSurrogateActor>(InLevel, AFastGeoSurrogateActor::StaticClass());
			if (const IWorldPartitionCell* Cell = InLevel->GetWorldPartitionRuntimeCell())
			{
				SurrogateActor->SetActorLocation(Cell->GetCellBounds().GetCenter());
			}
			SurrogateActor->SurrogateComponentDescriptors = MoveTemp(LevelSurrogateComponentDescriptors);
			InLevel->Actors.Add(SurrogateActor);
			check(SurrogateActor->GetLevel() == InLevel);

			for (int32 DescriptorIndex = 0; DescriptorIndex < SurrogateActor->SurrogateComponentDescriptors.Num(); ++DescriptorIndex)
			{
				const FFastGeoSurrogateComponentDescriptor& ComponentDesc = SurrogateActor->SurrogateComponentDescriptors[DescriptorIndex];
				UFastGeoSurrogateComponent* SurrogateComp = NewObject<UFastGeoSurrogateComponent>(SurrogateActor);
				SurrogateActor->AddInstanceComponent(SurrogateComp);
				SurrogateComp->DescriptorIndex = DescriptorIndex;
				SurrogateComp->SetSurrogateComponentDescriptor(ComponentDesc);
				SurrogateComp->SetupAttachment(SurrogateActor->GetRootComponent());
			}

			FastGeo->SurrogateActor = SurrogateActor;
		}

		// Finalize post-creation intialization
		FastGeo->OnCreated();
	}

	InLevel->Actors.Remove(nullptr);

	if (bIsDebugMode)
	{
		const FString Title = FString::Printf(TEXT("FastGeo Debug Mode: Level '%s'"), *InLevel->GetPathName());
		TArray<AActor*> EmptyIgnored;
		DumpDebugReport(*Title, EmptyIgnored, ActorReports, Stats);
	}
}

void UFastGeoWorldPartitionRuntimeCellTransformer::ForEachIgnoredComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const
{
	Super::ForEachIgnoredComponentClass(Func);

	const UFastGeoTransformerSettings& EffectiveSettings = GetSettings();

	for (const TSubclassOf<UActorComponent>& IgnoredRemainingComponentClass : EffectiveSettings.IgnoredRemainingComponentClasses)
	{
		if (!Func(IgnoredRemainingComponentClass))
		{
			return;
		}
	}
	for (const TSubclassOf<UActorComponent>& BuiltinIgnoredRemainingComponentClass : BuiltinIgnoredRemainingComponentClasses)
	{
		if (!Func(BuiltinIgnoredRemainingComponentClass))
		{
			return;
		}
	}
}

void UFastGeoWorldPartitionRuntimeCellTransformer::ForEachIgnoredExactComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const
{
	Super::ForEachIgnoredExactComponentClass(Func);

	const UFastGeoTransformerSettings& EffectiveSettings = GetSettings();

	for (const TSubclassOf<UActorComponent>& IgnoredRemainingExactComponentClass : EffectiveSettings.IgnoredRemainingExactComponentClasses)
	{
		if (!Func(IgnoredRemainingExactComponentClass))
		{
			return;
		}
	}
	for (const TSubclassOf<UActorComponent>& BuiltinIgnoredRemainingExactComponentClass : BuiltinIgnoredRemainingExactComponentClasses)
	{
		if (!Func(BuiltinIgnoredRemainingExactComponentClass))
		{
			return;
		}
	}
}

TMap<AActor*, TArray<AActor*>> UFastGeoWorldPartitionRuntimeCellTransformer::BuildActorsReferencesMap(const TArray<AActor*>& InActors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldPartitionRuntimeCellTransformer::BuildActorsReferencesMap);

	TMap<AActor*, TArray<AActor*>> ReferencedActors;
	TSet<UObject*> VisitedObjects;
	
	// Visit all actors properties and look for references to other actors
	for (AActor* ReferencingActor : InActors)
	{
		if (!IsValid(ReferencingActor))
		{
			continue;
		}

		if (CanAlwaysIgnoreActor(ReferencingActor))
		{
			continue;
		}

		// Editor-only actors won't exist at cook time, so their references should not prevent other actors from being transformed.
		if (ReferencingActor->IsEditorOnly())
		{
			continue;
		}

		VisitedObjects.Reset();
		VisitedObjects.Add(ReferencingActor);

		ReferencingActor->GetClass()->Visit(ReferencingActor, [&ReferencingActor, &ReferencedActors, &VisitedObjects](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
		{
			const FPropertyVisitorPath& Path = Context.Path;
			const FPropertyVisitorData& Data = Context.Data;
			const FProperty* Property = Path.Top().Property;

			// Step over editor only properties
			if (Property->IsEditorOnlyProperty())
			{
				return EPropertyVisitorControlFlow::StepOver;
			}

			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
			{
				if (UObject* PropertyObject = ObjectProperty->GetObjectPropertyValue(Data.PropertyData))
				{
					bool bWasAlreadyInSet;
					VisitedObjects.Add(PropertyObject, &bWasAlreadyInSet);

					if (bWasAlreadyInSet)
					{
						return EPropertyVisitorControlFlow::StepOver;
					}

					AActor* ReferencedActor = Cast<AActor>(PropertyObject);
					if (!ReferencedActor)
					{
						ReferencedActor = PropertyObject->GetTypedOuter<AActor>();
					}

					if (ReferencedActor && !ReferencedActor->HasAnyFlags(RF_ClassDefaultObject) && ReferencedActor != ReferencingActor)
					{
						ReferencedActors.FindOrAdd(ReferencedActor).Add(ReferencingActor);
					}

					// Constrain visitor to properties of objects that have ReferencingActor in their outer chain
					if (!PropertyObject->IsIn(ReferencingActor))
					{
						return EPropertyVisitorControlFlow::StepOver;
					}
				}
			}

			return EPropertyVisitorControlFlow::StepInto;
		}, FPropertyVisitorContext::EScope::ObjectRefs);
	}

	return ReferencedActors;
}

void UFastGeoWorldPartitionRuntimeCellTransformer::GatherTransformableActors(const TArray<AActor*>& InActors, const ULevel* InLevel, TArray<TPair<AActor*, FTransformableActor>>& OutTransformableActors, FTransformationStats& OutStats, TArray<FActorReport>* OutReports)
{
	struct FEvaluatedActor
	{
		bool bIsFullyTransformable = false;
		FFastGeoTransformResult Result = EFastGeoTransform::Allow;
		TOptional<FTransformableActor> TransformData;

		bool IsTransformable() const { return TransformData.IsSet(); }
		void RejectTransformation() { TransformData.Reset(); }
	};

	TMap<AActor*, FEvaluatedActor> EvaluatedActors;

	// Evaluate an actor and cache the result. Returns the cached entry.
	auto EvaluateActor = [this, &EvaluatedActors](AActor* InActor) -> FEvaluatedActor&
	{
		if (FEvaluatedActor* Cached = EvaluatedActors.Find(InActor))
		{
			return *Cached;
		}
		FEvaluatedActor& Entry = EvaluatedActors.Add(InActor);
		TArray<UActorComponent*> TempComponents;
		Entry.Result = CanTransformActor(InActor, Entry.bIsFullyTransformable, TempComponents);
		if (Entry.Result.GetResult() != EFastGeoTransform::Allow)
		{
			Entry.bIsFullyTransformable = false;
		}
		return Entry;
	};

	// Evaluate input actors and collect transformation results
	TMap<const AActor*, int32> ActorReportIndices;
	for (int32 ActorIndex = 0; ActorIndex < InActors.Num(); ++ActorIndex)
	{
		AActor* Actor = InActors[ActorIndex];
		if (IsValid(Actor) && !CanAlwaysIgnoreActor(Actor))
		{
			OutStats.TotalActorCount++;
			OutStats.TotalComponentCount += Actor->GetComponents().Num();

			TArray<FComponentReport>* ComponentReports = nullptr;
			if (OutReports)
			{
				ActorReportIndices.Add(Actor, OutReports->Num());
				FActorReport& ActorReport = OutReports->AddDefaulted_GetRef();
				ActorReport.Actor = Actor;
				ComponentReports = &ActorReport.ComponentReports;
			}

			bool bIsActorFullyTransformable;
			TArray<UActorComponent*> TransformableComponents;
			FFastGeoTransformResult ActorTransformResult = CanTransformActor(Actor, bIsActorFullyTransformable, TransformableComponents, ComponentReports);

			if (OutReports)
			{
				FActorReport& ActorReport = (*OutReports)[ActorReportIndices[Actor]];
				ActorReport.Result = ActorTransformResult.GetResult();
				ActorReport.Reason = ActorTransformResult.GetReason();
			}

			FEvaluatedActor& EvaluatedActor = EvaluatedActors.Add(Actor);
			EvaluatedActor.Result = MoveTemp(ActorTransformResult);

			if (EvaluatedActor.Result.GetResult() == EFastGeoTransform::Allow)
			{
				EvaluatedActor.bIsFullyTransformable = bIsActorFullyTransformable;
				// Sort components by pathname to ensure deterministic order
				TransformableComponents.Sort([](const UActorComponent& A, const UActorComponent& B)
				{
					return A.GetPathName() < B.GetPathName();
				});
				EvaluatedActor.TransformData.Emplace(FTransformableActor{ ActorIndex, bIsActorFullyTransformable, MoveTemp(TransformableComponents) });
			}
		}
	}

	TMap<AActor*, TArray<AActor*>> ReferencedActors = BuildActorsReferencesMap(InLevel->Actors);

	// Pre-evaluate all referencers so we don't mutate EvaluatedActors during iteration below
	for (const TPair<AActor*, TArray<AActor*>>& RefPair : ReferencedActors)
	{
		for (AActor* Referencer : RefPair.Value)
		{
			EvaluateActor(Referencer);
		}
	}

	for (TPair<AActor*, FEvaluatedActor>& Pair : EvaluatedActors)
	{
		AActor* Actor = Pair.Key;
		FEvaluatedActor& EvaluatedActor = Pair.Value;

		// Skip actors that were rejected, discarded, or only evaluated as referencers
		if (!EvaluatedActor.IsTransformable())
		{
			continue;
		}

		// Exclude actors that have:
		//  * Non FastGeo referencers
		//  * FastGeo referencers that are going to be only partially transformed
		if (const TArray<AActor*>* Referencers = ReferencedActors.Find(Actor))
		{
			AActor* const* FailingReferencerPtr = Referencers->FindByPredicate([&EvaluatedActors](AActor* InReferencer)
			{
				return !EvaluatedActors.FindChecked(InReferencer).bIsFullyTransformable;
			});

			if (FailingReferencerPtr)
			{
				AActor* FailingReferencer = *FailingReferencerPtr;
				const FEvaluatedActor& ReferencerEval = EvaluatedActors.FindChecked(FailingReferencer);
				const bool bReferencerIsTransformable = ReferencerEval.Result.GetResult() == EFastGeoTransform::Allow;
				const FString& ReferencerReason = ReferencerEval.Result.GetReason();
				FFastGeoTransformResult RejectResult(EFastGeoTransform::Reject, [Actor, FailingReferencer, bReferencerIsTransformable, &ReferencerReason]
				{
					const TCHAR* ReferencerDesc = bReferencerIsTransformable ? TEXT("non-fully transformed") : TEXT("non-transformable");
					return FString::Printf(TEXT("Actor '%s' is referenced by a %s actor ('%s'%s)"), *Actor->GetName(), ReferencerDesc, *FailingReferencer->GetName(), *(ReferencerReason.IsEmpty() ? FString() : FString::Printf(TEXT(": %s"), *ReferencerReason)));
				});

				if (OutReports)
				{
					FActorReport& Report = (*OutReports)[ActorReportIndices.FindChecked(Actor)];
					Report.Result = RejectResult.GetResult();
					Report.Reason = RejectResult.GetReason();
					Report.ComponentReports.Empty();
				}

				EvaluatedActor.RejectTransformation();
				continue;
			}
		}

		OutStats.FullyTransformableActorCount += EvaluatedActor.bIsFullyTransformable ? 1 : 0;
		OutStats.PartiallyTransformableActorCount += !EvaluatedActor.bIsFullyTransformable ? 1 : 0;
		OutStats.TransformedComponentCount += EvaluatedActor.TransformData->TransformableComponents.Num();
	}

	// Sort by pathname to ensure deterministic order
	for (TPair<AActor*, FEvaluatedActor>& Pair : EvaluatedActors)
	{
		AActor* Actor = Pair.Key;
		FEvaluatedActor& EvaluatedActor = Pair.Value;
		if (EvaluatedActor.IsTransformable())
		{
			OutTransformableActors.Emplace(Actor, MoveTemp(EvaluatedActor.TransformData.GetValue()));
		}
	}
	OutTransformableActors.Sort([](const TPair<AActor*, FTransformableActor>& A, const TPair<AActor*, FTransformableActor>& B)
	{
		return A.Key->GetPathName() < B.Key->GetPathName();
	});
}

void UFastGeoWorldPartitionRuntimeCellTransformer::DumpDebugReport(const TCHAR* Title, const TArray<AActor*>& IgnoredActors, const TArray<FActorReport>& ActorReports, const FTransformationStats& Stats)
{
	const int32 TotalSelected = IgnoredActors.Num() + Stats.TotalActorCount;

	UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparatorOuter);

	if (IgnoredActors.Num() > 0 && Stats.TotalActorCount == 0)
	{
		UE_LOGF(LogFastGeoStreaming, Log, "%ls: %d actors (all always ignored)", Title, IgnoredActors.Num());
	}
	else if (IgnoredActors.Num() > 0)
	{
		UE_LOGF(LogFastGeoStreaming, Log, "%ls: %d actors (%d always ignored)", Title, TotalSelected, IgnoredActors.Num());
	}
	else
	{
		UE_LOGF(LogFastGeoStreaming, Log, "%ls: %d actors", Title, TotalSelected);
	}

	UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparator);

	// Count entries per category
	int32 NumRejected = 0;
	int32 NumAllowed = 0;
	for (const FActorReport& Report : ActorReports)
	{
		if (Report.Result == EFastGeoTransform::Allow)
		{
			NumAllowed++;
		}
		else
		{
			NumRejected++;
		}
	}

	auto LogSectionHeader = [](const TCHAR* Title, int32 Count)
	{
		UE_LOGF(LogFastGeoStreaming, Log, "");
		UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparator);
		UE_LOGF(LogFastGeoStreaming, Log, "%ls (%d):", Title, Count);
		UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparator);
	};

	// Ignored
	if (IgnoredActors.Num() > 0)
	{
		LogSectionHeader(TEXT("Ignored"), IgnoredActors.Num());
		for (const AActor* Actor : IgnoredActors)
		{
			UE_LOGF(LogFastGeoStreaming, Log, "    * '%ls' (%ls)", *Actor->GetActorNameOrLabel(), *Actor->GetClass()->GetName());
		}
	}

	auto GetComponentResultPrefix = [](EFastGeoTransform Result) -> const TCHAR*
	{
		switch (Result)
		{
		case EFastGeoTransform::Allow:   return TEXT("Allow");
		case EFastGeoTransform::Reject:  return TEXT("Reject");
		case EFastGeoTransform::Discard: return TEXT("Discard");
		default:                         return TEXT("Unknown");
		}
	};

	auto LogComponentReports = [&GetComponentResultPrefix](const TArray<FComponentReport>& ComponentReports)
	{
		// Allowed components first, then rejected/discarded
		for (const FComponentReport& Comp : ComponentReports)
		{
			UE_CLOGF(Comp.Result == EFastGeoTransform::Allow, LogFastGeoStreaming, Log, "        - Allow: '%ls'", *Comp.Component->GetName());
		}
		for (const FComponentReport& Comp : ComponentReports)
		{
			UE_CLOGF(Comp.Result != EFastGeoTransform::Allow, LogFastGeoStreaming, Log, "        - %ls: '%ls'%ls", GetComponentResultPrefix(Comp.Result), *Comp.Component->GetName(), *FString(Comp.Reason.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" - %s"), *Comp.Reason)));
		}
	};

	// Rejected/Discarded
	if (NumRejected > 0)
	{
		LogSectionHeader(TEXT("Rejected"), NumRejected);
		for (const FActorReport& Report : ActorReports)
		{
			if (Report.Result != EFastGeoTransform::Allow)
			{
				UE_LOGF(LogFastGeoStreaming, Log, "    * '%ls' - %ls", *Report.Actor->GetActorNameOrLabel(), *Report.Reason);
				if (!Report.ComponentReports.IsEmpty())
				{
					LogComponentReports(Report.ComponentReports);
				}
			}
		}
	}

	// Allowed
	if (NumAllowed > 0)
	{
		LogSectionHeader(TEXT("Allowed"), NumAllowed);
		for (const FActorReport& Report : ActorReports)
		{
			if (Report.Result == EFastGeoTransform::Allow)
			{
				UE_LOGF(LogFastGeoStreaming, Log, "    * '%ls' (%d components)", *Report.Actor->GetActorNameOrLabel(), Report.ComponentReports.Num());
				LogComponentReports(Report.ComponentReports);
			}
		}
	}

	UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparator);

	Stats.DumpStats(TEXT("  -"));

	if (!FPhysScene::SupportsAsyncPhysicsStateCreation() || !FPhysScene::SupportsAsyncPhysicsStateDestruction())
	{
		UE_LOGF(LogFastGeoStreaming, Warning, "  - NOTE: FastGeoStreaming requires 'p.Chaos.EnableAsyncInitBody' to be enabled.");
	}

	UE_LOGF(LogFastGeoStreaming, Log, "%ls", FastGeo::LogSeparatorOuter);
}

void UFastGeoWorldPartitionRuntimeCellTransformer::FTransformationStats::DumpStats(const TCHAR* InPrefixString) const
{
	if (TotalActorCount)
	{
		const float FullyTransformedActorPercentage = TotalActorCount > 0 ? (100.f * FullyTransformableActorCount) / TotalActorCount : 0.f;
		const float PartiallyTransformedActorPercentage = TotalActorCount > 0 ? (100.f * PartiallyTransformableActorCount) / TotalActorCount : 0.f;
		const float TransformedComponentPercentage = TotalComponentCount > 0 ? (100.f * TransformedComponentCount) / TotalComponentCount : 0.f;
		const int32 NonTransformableActorCount = FMath::Max(0, TotalActorCount - FullyTransformableActorCount - PartiallyTransformableActorCount);
		const float NonTransformableActorPercentage = NonTransformableActorCount > 0 ? (100.f * NonTransformableActorCount) / TotalActorCount : 0.f;

		UE_CLOGF(FullyTransformableActorCount, LogFastGeoStreaming, Log, "%ls Transformable Actors (Full)    = %d (%3.1f%%)", InPrefixString, FullyTransformableActorCount, FullyTransformedActorPercentage);
		UE_CLOGF(PartiallyTransformableActorCount, LogFastGeoStreaming, Log, "%ls Transformable Actors (Partial) = %d (%3.1f%%)", InPrefixString, PartiallyTransformableActorCount, PartiallyTransformedActorPercentage);
		UE_CLOGF(TransformedComponentCount, LogFastGeoStreaming, Log, "%ls Transformable Components       = %d (%3.1f%%)", InPrefixString, TransformedComponentCount, TransformedComponentPercentage);
		UE_CLOGF(NonTransformableActorCount, LogFastGeoStreaming, Log, "%ls Non-Transformable Actors       = %d (%3.1f%%)", InPrefixString, NonTransformableActorCount, NonTransformableActorPercentage);
	}
}

FFastGeoTransformResult UFastGeoWorldPartitionRuntimeCellTransformer::IsAllowedActorClass(AActor* InActor) const
{
	const UFastGeoTransformerSettings& EffectiveSettings = GetSettings();
	UClass* ActorClass = InActor->GetClass();

	for (TSubclassOf<AActor> DisallowedActorClass : EffectiveSettings.DisallowedActorClasses)
	{
		if (ActorClass->IsChildOf(DisallowedActorClass))
		{
			return { EFastGeoTransform::Reject, [InActor, DisallowedActorClass] { return FString::Printf(TEXT("Actor %s class is child of a disallowed class (%s)"), *InActor->GetName(), *DisallowedActorClass->GetName()); } };
		}
	}

	for (TSubclassOf<AActor> DisallowedActorClass : BuiltinDisallowedActorClasses)
	{
		if (ActorClass->IsChildOf(DisallowedActorClass))
		{
			return { EFastGeoTransform::Reject, [InActor, DisallowedActorClass] { return FString::Printf(TEXT("Actor %s class is child of a built-in disallowed class (%s)"), *InActor->GetName(), *DisallowedActorClass->GetName()); } };
		}
	}

	for (TSubclassOf<AActor> DisallowedExactActorClass : EffectiveSettings.DisallowedExactActorClasses)
	{
		if (ActorClass == DisallowedExactActorClass)
		{
			return { EFastGeoTransform::Reject, [InActor, DisallowedExactActorClass] { return FString::Printf(TEXT("Actor %s class is a disallowed exact class (%s)"), *InActor->GetName(), *DisallowedExactActorClass->GetName()); } };
		}
	}

	for (TSubclassOf<AActor> AllowedActorClass : EffectiveSettings.AllowedActorClasses)
	{
		if (ActorClass->IsChildOf(AllowedActorClass))
		{
			return EFastGeoTransform::Allow;
		}
	}

	for (TSubclassOf<AActor> BuiltinAllowedActorClass : BuiltinAllowedActorClasses)
	{
		if (ActorClass->IsChildOf(BuiltinAllowedActorClass))
		{
			return EFastGeoTransform::Allow;
		}
	}

	for (TSubclassOf<AActor> AllowedExactActorClass : EffectiveSettings.AllowedExactActorClasses)
	{
		if (ActorClass == AllowedExactActorClass)
		{
			return EFastGeoTransform::Allow;
		}
	}

	// Special case where we allow an actor class if actor is tagged 'FastGeo'
	if (InActor->Tags.Contains(FastGeo::NAME_FastGeo))
	{
		return EFastGeoTransform::Allow;
	}

	return { EFastGeoTransform::Reject, [InActor, ActorClass] { return FString::Printf(TEXT("Actor %s class is an unsupported class (%s)"), *InActor->GetName(), *ActorClass->GetName()); } };
}

bool UFastGeoWorldPartitionRuntimeCellTransformer::IsComponentMobilitySupported(USceneComponent* InComponent, FString& OutReason) const
{
	bool bIsSupported = false;
	check(InComponent);

	const EComponentMobility::Type Mobility = InComponent->GetMobility();
	if (InComponent->IsA<UDecalComponent>())
	{
		// Allow all Mobility values for DecalComponent, as its mobility is Movable
		bIsSupported = true;
	}
	else if (InComponent->IsA<UPointLightComponent>() || InComponent->IsA<URectLightComponent>() || InComponent->IsA<USpotLightComponent>())
	{
		// Allow Stationary support LightComponents as Light Function Materials requires at least Stationary
		bIsSupported = (Mobility != EComponentMobility::Movable);
	}
	else
	{
		bIsSupported = (Mobility == EComponentMobility::Static);
	}

	if (!bIsSupported)
	{
		OutReason = FString::Printf(TEXT("(%s) has unsupported Mobility: %s"), *InComponent->GetClass()->GetName(), *StaticEnum<EComponentMobility::Type>()->GetNameStringByValue((int64)Mobility));
	}

	return bIsSupported;
}

FFastGeoTransformResult UFastGeoWorldPartitionRuntimeCellTransformer::CanTransformActor(AActor* InActor, bool& bOutIsActorFullyTransformable, TArray<UActorComponent*>& OutTransformableComponents, TArray<FComponentReport>* OutComponentReports) const
{
	bOutIsActorFullyTransformable = false;

	FFastGeoTransformResult AllowedActorClassResult = IsAllowedActorClass(InActor);
	if (AllowedActorClassResult.GetResult() != EFastGeoTransform::Allow)
	{
		return AllowedActorClassResult;
	}

	// Reject actors containing disallowed component classes
	{
		const UFastGeoTransformerSettings& EffectiveSettings = GetSettings();
		TArray<UActorComponent*> ActorComponents;
		InActor->GetComponents(ActorComponents);

		for (const UActorComponent* ActorComponent : ActorComponents)
		{
			const UClass* ComponentClass = ActorComponent->GetClass();

			for (TSubclassOf<UActorComponent> DisallowedComponentClass : EffectiveSettings.DisallowedActorsContainingComponentClasses)
			{
				if (ComponentClass->IsChildOf(DisallowedComponentClass))
				{
					return { EFastGeoTransform::Reject, [InActor, ActorComponent, DisallowedComponentClass] { return FString::Printf(TEXT("Actor %s contains a component %s of disallowed class (%s)"), *InActor->GetName(), *FastGeo::GetComponentShortName(ActorComponent), *DisallowedComponentClass->GetName()); } };
				}
			}

			for (TSubclassOf<UActorComponent> DisallowedExactComponentClass : EffectiveSettings.DisallowedActorsContainingExactComponentClasses)
			{
				if (ComponentClass == DisallowedExactComponentClass)
				{
					return { EFastGeoTransform::Reject, [InActor, ActorComponent, DisallowedExactComponentClass] { return FString::Printf(TEXT("Actor %s contains a component %s of disallowed exact class (%s)"), *InActor->GetName(), *FastGeo::GetComponentShortName(ActorComponent), *DisallowedExactComponentClass->GetName()); } };
				}
			}
		}
	}

	FString Reason;

	if (InActor->ActorHasTag(NAME_CellTransformerIgnoreActor))
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is tagged '%s'"), *InActor->GetName(), *NAME_CellTransformerIgnoreActor.ToString()); } };
	}

	if (InActor->ActorHasTag(FastGeo::NAME_NoFastGeo))
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is tagged '%s'"), *InActor->GetName(), *FastGeo::NAME_NoFastGeo.ToString()); } };
	}

	if (!IsActorTransformable(InActor, Reason))
	{
		return { EFastGeoTransform::Reject, [InActor, Reason] { return FString::Printf(TEXT("Actor %s [%s]"), *InActor->GetName(), *Reason); } };
	}

	// Reject actors that are never going to be streamed
	// This test is only useful for the FastGeo debug mode, as normally during PIE & cook, only streamed actors will be processed by the cell transformer.
	if (!InActor->GetLevel()->GetWorldPartitionRuntimeCell())
	{
		if (!InActor->GetIsSpatiallyLoaded())
		{
			bool bHasRuntimeDataLayers = Algo::AnyOf(InActor->GetDataLayerInstances(), [](const UDataLayerInstance* DataLayer) { return DataLayer->IsRuntime(); });
			if (!bHasRuntimeDataLayers)
			{
				return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is non-spatially loaded"), *InActor->GetName()); } };
			}
		}
	}

	if (InActor->GetIsReplicated())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is replicated"), *InActor->GetName()); } };
	}

	if (!InActor->GetRootComponent())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s RootComponent is invalid"), *InActor->GetName()); } };
	}

	Reason.Reset();
	if (!IsComponentMobilitySupported(InActor->GetRootComponent(), Reason))
	{
		return { EFastGeoTransform::Reject, [InActor, Reason] { return FString::Printf(TEXT("Actor %s RootComponent %s"), *InActor->GetName(), *Reason); } };
	}

	if (InActor->IsEditorOnly())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is editor-only"), *InActor->GetName()); } };
	}

	if (InActor->Children.Num())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s has children"), *InActor->GetName()); } };
	}

	if (InActor->IsChildActor())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is a child actor"), *InActor->GetName()); } };
	}

	if (IsBlueprintActorWithLogic(InActor))
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s is a Blueprint Actor with logic"), *InActor->GetName()); } };
	}

	// Gather transformable components
	TArray<UActorComponent*> TransformResults[EnumToIndex(EFastGeoTransform::MAX)];
	InActor->ForEachComponent<UActorComponent>(false, [this, &TransformResults, OutComponentReports](UActorComponent* Component)
	{
		FFastGeoTransformResult TransformComponentResult = CanTransformComponent(Component);
		TransformResults[TransformComponentResult.GetResultIndex()].Add(Component);
		if (OutComponentReports)
		{
			OutComponentReports->Add({ Component, TransformComponentResult.GetResult(), TransformComponentResult.GetReason() });
		}
	});

	if (TransformResults[EnumToIndex(EFastGeoTransform::Allow)].IsEmpty())
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Actor %s has no transformable components"), *InActor->GetName()); } };
	}

	// If actor contains only discardable or transformable components, we can actually get rid of it
	bOutIsActorFullyTransformable = TransformResults[EnumToIndex(EFastGeoTransform::Reject)].IsEmpty();

	// Reject partial transformation when transformable components are default subobjects.
	// When an actor is only partially transformed, it persists in the cooked build. The actor's
	// constructor will recreate any default subobject components that were removed during
	// transformation, producing phantom geometry with default properties
	// (e.g. a DecalComponent with no material assigned).
	if (!bOutIsActorFullyTransformable)
	{
		for (UActorComponent* Component : TransformResults[EnumToIndex(EFastGeoTransform::Allow)])
		{
			if (Component->HasAnyFlags(RF_DefaultSubObject))
			{
				return { EFastGeoTransform::Reject, [InActor, Component] { return FString::Printf(TEXT("Actor %s cannot be partially transformed: component %s is a default subobject that would be recreated by the constructor at load time"), *InActor->GetName(), *FastGeo::GetComponentShortName(Component)); } };
			}
		}
	}

	// Reject transformation when a decal has bDestroyOwnerAfterFade and there are
	// other non-discardable components. FastGeo has no concept of source actor grouping
	// and cannot cascade destruction to sibling components when the decal fade completes.
	// Also check rejected decals: if the original actor self-destructs after fade,
	// any FastGeo components from the same actor would be orphaned.
	{
		const int32 NonDiscardableCount = TransformResults[EnumToIndex(EFastGeoTransform::Allow)].Num() + TransformResults[EnumToIndex(EFastGeoTransform::Reject)].Num();
		if (NonDiscardableCount > 1)
		{
			auto HasDestroyAfterFadeDecal = [](const TArray<UActorComponent*>& Components, UActorComponent*& OutComponent)
			{
				for (UActorComponent* Component : Components)
				{
					if (UDecalComponent* DecalComp = Cast<UDecalComponent>(Component))
					{
						if (DecalComp->bDestroyOwnerAfterFade && (DecalComp->GetFadeDuration() > 0.0f || DecalComp->GetFadeStartDelay() > 0.0f))
						{
							OutComponent = Component;
							return true;
						}
					}
				}
				return false;
			};

			UActorComponent* FadingDecal = nullptr;
			if (HasDestroyAfterFadeDecal(TransformResults[EnumToIndex(EFastGeoTransform::Allow)], FadingDecal) || HasDestroyAfterFadeDecal(TransformResults[EnumToIndex(EFastGeoTransform::Reject)], FadingDecal))
			{
				return { EFastGeoTransform::Reject, [InActor, FadingDecal] { return FString::Printf(TEXT("Actor %s has component %s with bDestroyOwnerAfterFade which cannot cascade destruction to sibling components"), *InActor->GetName(), *FastGeo::GetComponentShortName(FadingDecal)); } };
			}
		}
	}

	// BP actors can't be partially transformed (rerun CS will be called in PIE when registering
	// the component and also called when registering components during cook/save of the level).
	if (UBlueprint::GetBlueprintFromClass(InActor->GetClass()) && !bOutIsActorFullyTransformable)
	{
		return { EFastGeoTransform::Reject, [InActor] { return FString::Printf(TEXT("Blueprint actor %s has non-transformable components"), *InActor->GetName()); } };
	}

	Reason.Reset();
	if (!IsFullyTransformedActorDeletable(InActor, Reason))
	{
		bOutIsActorFullyTransformable = false;
		if (UBlueprint::GetBlueprintFromClass(InActor->GetClass()))
		{
			return { EFastGeoTransform::Reject, [InActor, Reason] { return FString::Printf(TEXT("Blueprint actor %s can't be deleted [%s]"), *InActor->GetName(), *Reason); } };
		}
	}

	OutTransformableComponents = MoveTemp(TransformResults[EnumToIndex(EFastGeoTransform::Allow)]);
	return EFastGeoTransform::Allow;
}

FFastGeoTransformResult UFastGeoWorldPartitionRuntimeCellTransformer::IsAllowedComponentClass(UActorComponent* InComponent) const
{
	const UFastGeoTransformerSettings& EffectiveSettings = GetSettings();

	FFastGeoElementType FastGeoComponentType = FastGeo::GetFastGeoComponentType(InComponent->GetClass());
	if (!FastGeoComponentType.IsValid())
	{
		return { EFastGeoTransform::Reject, [InComponent] { return FString::Printf(TEXT("Component %s class is unsupported (%s)"), *FastGeo::GetComponentShortName(InComponent), *InComponent->GetClass()->GetName()); } };
	}

	UClass* ComponentClass = InComponent->GetClass();
	for (TSubclassOf<UActorComponent> DisallowedComponentClass : EffectiveSettings.DisallowedComponentClasses)
	{
		if (ComponentClass->IsChildOf(DisallowedComponentClass))
		{
			return { EFastGeoTransform::Reject, [InComponent, DisallowedComponentClass] { return FString::Printf(TEXT("Component %s class is child of a disallowed class (%s)"), *FastGeo::GetComponentShortName(InComponent), *DisallowedComponentClass->GetName()); } };
		}
	}
	
	for (TSubclassOf<UActorComponent> BuiltinDisallowedComponentClass : BuiltinDisallowedComponentClasses)
	{
		if (ComponentClass->IsChildOf(BuiltinDisallowedComponentClass))
		{
			return { EFastGeoTransform::Reject, [InComponent, BuiltinDisallowedComponentClass] { return FString::Printf(TEXT("Component %s class is child of a built-in disallowed class (%s)"), *FastGeo::GetComponentShortName(InComponent), *BuiltinDisallowedComponentClass->GetName()); } };
		}
	}

	for (TSubclassOf<UActorComponent> DisallowedExactComponentClass : EffectiveSettings.DisallowedExactComponentClasses)
	{
		if (ComponentClass == DisallowedExactComponentClass)
		{
			return { EFastGeoTransform::Reject, [InComponent, DisallowedExactComponentClass] { return FString::Printf(TEXT("Component %s class is a disallowed exact class (%s)"), *FastGeo::GetComponentShortName(InComponent), *DisallowedExactComponentClass->GetName()); } };
		}
	}

	for (TSubclassOf<UActorComponent> AllowedComponentClass : EffectiveSettings.AllowedComponentClasses)
	{
		if (ComponentClass->IsChildOf(AllowedComponentClass))
		{
			return EFastGeoTransform::Allow;
		}
	}

	for (TSubclassOf<UActorComponent> BuiltinAllowedComponentClass : BuiltinAllowedComponentClasses)
	{
		if (ComponentClass->IsChildOf(BuiltinAllowedComponentClass))
		{
			return EFastGeoTransform::Allow;
		}
	}
	
	for (TSubclassOf<UActorComponent> AllowedExactComponentClass : EffectiveSettings.AllowedExactComponentClasses)
	{
		if (ComponentClass == AllowedExactComponentClass)
		{
			return EFastGeoTransform::Allow;
		}
	}

	return { EFastGeoTransform::Reject, [InComponent, ComponentClass] { return FString::Printf(TEXT("Component %s class is an unsupported class (%s)"), *FastGeo::GetComponentShortName(InComponent), *ComponentClass->GetName()); } };
};

FFastGeoTransformResult UFastGeoWorldPartitionRuntimeCellTransformer::CanTransformComponent(UActorComponent* InComponent) const
{
	if (InComponent->IsEditorOnly())
	{
		return { EFastGeoTransform::Discard, [InComponent] { return FString::Printf(TEXT("Component %s is editor-only"), *FastGeo::GetComponentShortName(InComponent)); } };
	}

	if (CanIgnoreComponent(InComponent))
	{
		return { EFastGeoTransform::Discard, [InComponent] { return FString::Printf(TEXT("Component %s class is ignored (%s)"), *FastGeo::GetComponentShortName(InComponent), *InComponent->GetClass()->GetName()); } };
	}

	USceneComponent* SceneComponent = Cast<USceneComponent>(InComponent);
	if (!SceneComponent)
	{
		return { EFastGeoTransform::Reject, [InComponent] { return FString::Printf(TEXT("Component %s class is not allowed (%s)"), *FastGeo::GetComponentShortName(InComponent), *InComponent->GetClass()->GetName()); } }; //-V522
	}

	const FFastGeoTransformResult AllowedComponentClassResult = IsAllowedComponentClass(InComponent);
	if (AllowedComponentClassResult.GetResult() != EFastGeoTransform::Allow)
	{
		return AllowedComponentClassResult;
	}

	FString Reason;
	if (!IsComponentTransformable(SceneComponent, Reason))
	{
		return { EFastGeoTransform::Reject, [SceneComponent, Reason] { return FString::Printf(TEXT("Component %s [%s]"), *FastGeo::GetComponentShortName(SceneComponent), *Reason); } };
	}

	Reason.Reset();
	if (!IsComponentMobilitySupported(SceneComponent, Reason))
	{
		return { EFastGeoTransform::Reject, [SceneComponent, Reason] { return FString::Printf(TEXT("Component %s %s"), *FastGeo::GetComponentShortName(SceneComponent), *Reason); } };
	}

	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InComponent);
	if (PrimitiveComponent && PrimitiveComponent->GetLODParentPrimitive())
	{
		return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has a valid LOD Parent Primitive"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
	}

	if (PrimitiveComponent && PrimitiveComponent->BodyInstance.bNotifyRigidBodyCollision)
	{
		return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has the 'Simulation Generates Hit Events' flag set"), *FastGeo::GetComponentShortName(PrimitiveComponent)); }};
	}

	bool bIsSceneComponentVisible = SceneComponent->IsVisible() && !SceneComponent->GetOwner()->IsHidden();
	bool bIsPrimitiveVisible = PrimitiveComponent && (PrimitiveComponent->bCastHiddenShadow || PrimitiveComponent->bAffectIndirectLightingWhileHidden || PrimitiveComponent->bRayTracingFarField);
	bool bShouldAddToRenderScene = bIsSceneComponentVisible || bIsPrimitiveVisible;

	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
	{
		if (!StaticMeshComponent->GetStaticMesh())
		{
			return { EFastGeoTransform::Discard, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has an invalid static mesh"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		// Make sure BodyInstance CollisionEnabled is updated first before testing below
		StaticMeshComponent->UpdateCollisionFromStaticMesh();
	}

	if (UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(PrimitiveComponent))
	{
		if (InstancedStaticMeshComponent->GetNumInstances() == 0)
		{
			return { EFastGeoTransform::Discard, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has no instances"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}
	}

	if (UHierarchicalInstancedStaticMeshComponent* HierarchicalInstancedStaticMeshComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(PrimitiveComponent))
	{
		// FastGeo doesn't really support HISMC. These components get converted to ISMC.
		// However, we can afford to convert nanite HISMC as all the LODing logic is performed by Nanite.
		// We also allow the transformation of HISMC which are using a mesh with a single LOD - so in effect it's handled as an ISMC.
		if (!HierarchicalInstancedStaticMeshComponent->GetStaticMesh()->IsNaniteEnabled() || HierarchicalInstancedStaticMeshComponent->IsForceDisableNanite())
		{
			if (HierarchicalInstancedStaticMeshComponent->GetStaticMesh()->GetNumLODs() > 1)
			{
				return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Hierarchical instanced static mesh component %s has multiple LODs"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
			}
		}
	}

	if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(PrimitiveComponent))
	{
		if (!SkinnedMeshComponent->GetSkinnedAsset())
		{
			return { EFastGeoTransform::Discard, [PrimitiveComponent] { return FString::Printf(TEXT("Skinned mesh component %s has an invalid skinned asset"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		if (SkinnedMeshComponent->LeaderPoseComponent.IsValid())
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Skinned mesh component %s has a leader pose component"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		if (SkinnedMeshComponent->GetCollisionEnabled() != ECollisionEnabled::NoCollision)
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Skinned mesh component %s has collisions enabled"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		if (SkinnedMeshComponent->IsNavigationRelevant())
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Skinned mesh component %s is navigation relevant"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		bShouldAddToRenderScene &= !SkinnedMeshComponent->bHideSkin;
	}

	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimitiveComponent))
	{
		if ((SkeletalMeshComponent->GetAnimationMode() == EAnimationMode::AnimationSingleNode && SkeletalMeshComponent->AnimationData.AnimToPlay != nullptr) ||
			(SkeletalMeshComponent->GetAnimationMode() == EAnimationMode::AnimationBlueprint && SkeletalMeshComponent->AnimClass != nullptr) ||
			(SkeletalMeshComponent->GetAnimationMode() == EAnimationMode::AnimationCustomMode))
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Skeletal mesh component %s is animated"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}
	}

	if (UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = Cast<UInstancedSkinnedMeshComponent>(PrimitiveComponent))
	{
		// FastGeo accepts static-pose ISKMCs and ISKMCs driven by a GPU-only TransformProvider
		// (UTransformProviderData::IsGpuOnly() == true). CPU-driven providers are rejected because
		// the cell transformer drops the owning component's game-thread tick.
		//
		// Note: ISKMC is structurally GPU-driven - FInstancedSkinnedMeshSceneProxyDesc::CreateMeshObject
		// only returns Nanite/Static/GPUSkin mesh objects, never CPU skin. The platform-specific dispatch
		// between Nanite and GPU-skin happens at runtime, so we deliberately don't gate on Nanite here.

		if (InstancedSkinnedMeshComponent->GetInstanceCount() == 0)
		{
			return { EFastGeoTransform::Discard, [PrimitiveComponent] { return FString::Printf(TEXT("Instanced skinned mesh component %s has no instances"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		const UTransformProviderData* TransformProvider = InstancedSkinnedMeshComponent->GetTransformProvider();
		if (TransformProvider && TransformProvider->IsEnabled() && !TransformProvider->IsGpuOnly())
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Instanced skinned mesh component %s uses a CPU-driven TransformProvider (only GPU-only providers are supported by FastGeo)"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}
	}

	if (PrimitiveComponent)
	{
		const bool bIsCollisionEnabled = FastGeo::IsCollisionEnabled(PrimitiveComponent);

		// If collision is enabled, only allow if async physics state creation and destruction are supported 
		if (bIsCollisionEnabled && (!PrimitiveComponent->AllowsAsyncPhysicsStateCreation() || !PrimitiveComponent->AllowsAsyncPhysicsStateDestruction()))
		{
			return { EFastGeoTransform::Reject, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has collision enabled but doesn't allow asynchronous physics state creation/destruction"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}

		// Disallow transform if collision is disabled and component doesn't need to be added to the render scene
		if (!bIsCollisionEnabled && !bShouldAddToRenderScene)
		{
			return { EFastGeoTransform::Discard, [PrimitiveComponent] { return FString::Printf(TEXT("Component %s has no collision and is not visible"), *FastGeo::GetComponentShortName(PrimitiveComponent)); } };
		}
	}
	else if (!bIsSceneComponentVisible)
	{
		return { EFastGeoTransform::Discard, [SceneComponent] { return FString::Printf(TEXT("Component %s is not visible"), *FastGeo::GetComponentShortName(SceneComponent)); } };
	}

	if (CanIgnoreComponent(InComponent))
	{
		return { EFastGeoTransform::Discard, [InComponent] { return FString::Printf(TEXT("Component %s class is ignored (%s)"), *FastGeo::GetComponentShortName(InComponent), *InComponent->GetClass()->GetName()); } };
	}

	return EFastGeoTransform::Allow;
}

void UFastGeoWorldPartitionRuntimeCellTransformer::PreEditChange(FProperty* InPropertyAboutToChange)
{
	FastGeo::GPackageWasDirty = GetPackage()->IsDirty();
	Super::PreEditChange(InPropertyAboutToChange);
}

void UFastGeoWorldPartitionRuntimeCellTransformer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UFastGeoWorldPartitionRuntimeCellTransformer, bDebugMode) || 
		 PropertyName == GET_MEMBER_NAME_CHECKED(UFastGeoWorldPartitionRuntimeCellTransformer, bDebugSelectionMode)) && !FastGeo::GPackageWasDirty)
	{
		GetPackage()->ClearDirtyFlag();
	}
}

bool FFastGeoTransformResult::bShouldCollectReasons = false;

FFastGeoTransformResult::FFastGeoTransformResult(EFastGeoTransform InTransformResult, const TCHAR* InFailureReason)
	: TransformResult(InTransformResult)
{
	if (TransformResult != EFastGeoTransform::Allow && InFailureReason && bShouldCollectReasons)
	{
		Reason = InFailureReason;
	}
}

FFastGeoTransformResult::FFastGeoTransformResult(EFastGeoTransform InTransformResult, TFunctionRef<FString()> InFailureReasonFunc)
	: TransformResult(InTransformResult)
{
	if (TransformResult != EFastGeoTransform::Allow && bShouldCollectReasons)
	{
		Reason = InFailureReasonFunc();
	}
}

#endif

#undef LOCTEXT_NAMESPACE
