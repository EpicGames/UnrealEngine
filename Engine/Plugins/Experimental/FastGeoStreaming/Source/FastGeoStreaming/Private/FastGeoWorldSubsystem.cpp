// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoWorldSubsystem.h"
#include "FastGeoContainer.h"
#include "FastGeoLog.h"
#include "FastGeoPrimitiveComponent.h"
#include "FastGeoStreamingModule.h"
#include "FastGeoSurrogateActor.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "ComponentReregisterContext.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "Engine/GameViewportClient.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#if !UE_BUILD_SHIPPING
#include "FastGeoPhysicsBodyInstanceOwner.h"
#include "FastGeoSurrogateComponent.h"
#include "FastGeoSurrogateBodyInstanceIndex.h"
#include "FastGeoStaticMeshComponent.h"
#include "FastGeoInstancedStaticMeshComponent.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "GameFramework/PlayerController.h"
#include "DrawDebugHelpers.h"
#endif

#define LOCTEXT_NAMESPACE "FastGeoWorldSubsystem"

static FName NAME_FastGeoColorHandler(TEXT("FastGeo"));
bool UFastGeoWorldSubsystem::bEnableDebugView = false;

#if WITH_EDITOR
UFastGeoWorldSubsystem::FOnComponentsPreRecreate UFastGeoWorldSubsystem::ComponentsPreRecreateEvent;
#endif

namespace FastGeo
{
	static float GAsyncRenderStateTaskTimeBudgetMS = 0.0f;
	static FAutoConsoleVariableRef CVarAsyncRenderStateTaskTimeBudgetMS(
		TEXT("FastGeo.AsyncRenderStateTask.TimeBudgetMS"),
		GAsyncRenderStateTaskTimeBudgetMS,
		TEXT("Maximum time budget in milliseconds for the async render state tasks (0 = no time limit)"),
		ECVF_Default);

	static float GSyncPSORecreateRenderStateTimeBudgetMS = 0.0f;
	static FAutoConsoleVariableRef CVarSyncPSORecreateRenderStateTimeBudgetMS(
		TEXT("FastGeo.SyncPSORecreateRenderState.TimeBudgetMS"),
		GSyncPSORecreateRenderStateTimeBudgetMS,
		TEXT("Maximum time budget in milliseconds for PSO recreate render state work per frame in sync mode (0 = no time limit)"),
		ECVF_Default);

	static int32 GAsyncRenderStateTaskMaxNumComponentsToProcess = 0;
	static FAutoConsoleVariableRef CVarAsyncRenderStateTaskMaxNumComponentsToProcess(
		TEXT("FastGeo.AsyncRenderStateTask.MaxNumComponentsToProcess"),
		GAsyncRenderStateTaskMaxNumComponentsToProcess,
		TEXT("Maximum number of components to process (0 = no component limit)"),
		ECVF_Default);

	static bool GAllowSurrogateComponents = true;
	static FAutoConsoleVariableRef CVarAllowSurrogateComponents(
		TEXT("FastGeo.AllowSurrogateComponents"),
		GAllowSurrogateComponents,
		TEXT("Allows the use of generated FastGeo surrogate components.\n")
		TEXT("Must be set at startup; cannot be changed during runtime.\n")
		TEXT("Requires data transformed with 'bGenerateSurrogateComponents' enabled."),
		ECVF_ReadOnly);

	// RAII guard that stamps a std::atomic<bool> on construction and restores its prior value on destruction.
	// Mirrors TGuardValue<bool>'s save-and-restore semantics for code paths where the underlying flag
	// is read concurrently from worker threads and therefore cannot be a plain bool.
	struct FAtomicBoolGuard
	{
		FAtomicBoolGuard(std::atomic<bool>& InTarget, bool InNewValue)
			: Target(InTarget)
			, OldValue(InTarget.exchange(InNewValue, std::memory_order_release))
		{}

		~FAtomicBoolGuard()
		{
			Target.store(OldValue, std::memory_order_release);
		}

		FAtomicBoolGuard(const FAtomicBoolGuard&) = delete;
		FAtomicBoolGuard& operator=(const FAtomicBoolGuard&) = delete;

	private:
		std::atomic<bool>& Target;
		bool OldValue;
	};

#if !UE_BUILD_SHIPPING
	static bool GShowFastGeo = true;
	FAutoConsoleCommand GShowFastGeoCommand(
		TEXT("FastGeo.Show"),
		TEXT("Turn on/off rendering of FastGeo."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			GShowFastGeo = (Args.Num() != 1) || (Args[0] != TEXT("0"));
			for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
			{
				UFastGeoContainer* FastGeo = *It;
				if (FastGeo->GetWorld())
				{
					FastGeo->ForEachComponentCluster([](FFastGeoComponentCluster& ComponentCluster)
					{
						ComponentCluster.UpdateVisibility();
					});
				}
			}
		})
	);

	static bool GDebugTraceHUD = false;
	static FAutoConsoleVariableRef CVarDebugTraceHUD(
		TEXT("FastGeo.Debug.TraceHUD"),
		GDebugTraceHUD,
		TEXT("When enabled, performs a line trace from the camera and displays FastGeo component/container info on screen."),
		ECVF_Default);
#endif

}

CSV_DEFINE_CATEGORY(FastGeo, true);

bool UFastGeoWorldSubsystem::IsEnableDebugView()
{
	return bEnableDebugView;
}

#if !UE_BUILD_SHIPPING
bool UFastGeoWorldSubsystem::IsFastGeoVisible()
{
	return FastGeo::GShowFastGeo;
}
#endif

bool UFastGeoWorldSubsystem::ShouldAllowSurrogateComponents()
{
	return FastGeo::GAllowSurrogateComponents;
}

UFastGeoWorldSubsystem::UFastGeoWorldSubsystem()
{
	if (!FFastGeoStreamingModule::IsFastGeoEnabled())
	{
		return;
	}

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER

	auto UpdatePrimitivesColor = []()
	{
		for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UFastGeoContainer* FastGeo = *It;
			UWorld* World = FastGeo->GetWorld();
			if (World && World->IsGameWorld())
			{
				FastGeo->ForEachComponentCluster([](const FFastGeoComponentCluster& ComponentCluster)
				{
					ComponentCluster.ForEachComponent<FFastGeoPrimitiveComponent>([](const FFastGeoPrimitiveComponent& Component)
					{
						if (FPrimitiveSceneProxy* SceneProxy = Component.GetSceneProxy())
						{
							SceneProxy->SetPrimitiveColor_GameThread(Component.GetDebugColor());
						}
					});
				});
			}
		}
	};

	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<UFastGeoWorldSubsystem>(this))
	{
		FActorPrimitiveColorHandler::FPrimitiveColorHandler FastGeoColorHandler;
		FastGeoColorHandler.HandlerName = NAME_FastGeoColorHandler;
		FastGeoColorHandler.HandlerText = LOCTEXT("FastGeo", "FastGeo");
		FastGeoColorHandler.HandlerToolTipText = LOCTEXT("FastGeoColor_ToopTip", "Colorize FastGeo primitives: FastGeo [Blue], Non-FastGeo [Red]");
		FastGeoColorHandler.bAvailalbleInEditor = false;
		FastGeoColorHandler.GetColorFunc = [](const UPrimitiveComponent* InPrimitiveComponent)
		{
			return FLinearColor::Red;
		};
		FastGeoColorHandler.ActivateFunc = [&UpdatePrimitivesColor]()
		{
			bEnableDebugView = true;
			UpdatePrimitivesColor();
		};
		FastGeoColorHandler.DeactivateFunc = [&UpdatePrimitivesColor]()
		{
			bEnableDebugView = false;
			UpdatePrimitivesColor();
		};
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(FastGeoColorHandler);
	}
#endif
}

UFastGeoWorldSubsystem::~UFastGeoWorldSubsystem()
{
	if (!FFastGeoStreamingModule::IsFastGeoEnabled())
	{
		return;
	}

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<UFastGeoWorldSubsystem>(this))
	{
		FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_FastGeoColorHandler);
	}
#endif
}

bool UFastGeoWorldSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return FFastGeoStreamingModule::IsFastGeoEnabled() && Super::DoesSupportWorldType(WorldType);
}

void UFastGeoWorldSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UWorld* World = GetWorld();

	FWorldDelegates::OnWorldCleanup.AddUObject(this, &UFastGeoWorldSubsystem::OnWorldCleanup);
	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextCreated().AddUObject(this, &UFastGeoWorldSubsystem::OnGlobalComponentReregisterContextCreated);
	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextDestroyed().AddUObject(this, &UFastGeoWorldSubsystem::OnGlobalComponentReregisterContextDestroyed);
#if WITH_EDITOR
	FWorldDelegates::OnPreRecreateScene.AddUObject(this, &UFastGeoWorldSubsystem::OnPreRecreateScene);
#endif

	// Register streaming delegates for all game worlds, not just partitioned ones.
	// A non-partitioned world can host Level Instances referencing partitioned maps
	// with FastGeo content. The streaming cells load into the outer world's level list.
	if (World->IsGameWorld())
	{
		World->OnAllLevelsChanged().AddUObject(this, &UFastGeoWorldSubsystem::OnUpdateLevelStreaming);
		World->OnAddLevelToWorldExtension().AddUObject(this, &UFastGeoWorldSubsystem::OnAddLevelToWorldExtension);
		World->OnRemoveLevelFromWorldExtension().AddUObject(this, &UFastGeoWorldSubsystem::OnRemoveLevelFromWorldExtension);
		FWorldDelegates::LevelComponentsUpdated.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelComponentsUpdated);
		FWorldDelegates::LevelComponentsCleared.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelComponentsCleared);
#if DO_CHECK
		FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelAddedToWorld);
		FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelRemovedFromWorld);
#endif
		Handle_OnLevelStreamingStateChanged = FLevelStreamingDelegates::OnLevelStreamingStateChanged.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelStreamingStateChanged);
		Handle_OnLevelBeginAddToWorld = FLevelStreamingDelegates::OnLevelBeginMakingVisible.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelStartedAddToWorld);
		Handle_OnLevelBeginRemoveFromWorld = FLevelStreamingDelegates::OnLevelBeginMakingInvisible.AddUObject(this, &UFastGeoWorldSubsystem::OnLevelStartedRemoveFromWorld);

		if (UWorldPartitionHLODRuntimeSubsystem* HLODSubsystem = World->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>())
		{
			Handle_OnForEachHLODObjectInCell = HLODSubsystem->GetForEachHLODObjectInCellEvent().AddUObject(this, &UFastGeoWorldSubsystem::ForEachHLODObjectInCell);
		}
	}
}

void UFastGeoWorldSubsystem::Deinitialize()
{
	UWorld* World = GetWorld();

	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextCreated().RemoveAll(this);
	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextDestroyed().RemoveAll(this);
#if WITH_EDITOR
	FWorldDelegates::OnPreRecreateScene.RemoveAll(this);
#endif

	if (World->IsGameWorld())
	{
		World->OnAllLevelsChanged().RemoveAll(this);
		World->OnAddLevelToWorldExtension().RemoveAll(this);
		World->OnRemoveLevelFromWorldExtension().RemoveAll(this);
		FWorldDelegates::LevelComponentsUpdated.RemoveAll(this);
		FWorldDelegates::LevelComponentsCleared.RemoveAll(this);
#if DO_CHECK
		FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
#endif
		FLevelStreamingDelegates::OnLevelStreamingStateChanged.Remove(Handle_OnLevelStreamingStateChanged);
		FLevelStreamingDelegates::OnLevelBeginMakingVisible.Remove(Handle_OnLevelBeginAddToWorld);
		FLevelStreamingDelegates::OnLevelBeginMakingInvisible.Remove(Handle_OnLevelBeginRemoveFromWorld);
		Handle_OnLevelStreamingStateChanged.Reset();
		Handle_OnLevelBeginAddToWorld.Reset();
		Handle_OnLevelBeginRemoveFromWorld.Reset();

		if (Handle_OnForEachHLODObjectInCell.IsValid())
		{
			if (UWorldPartitionHLODRuntimeSubsystem* HLODSubsystem = World->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>())
			{
				HLODSubsystem->GetForEachHLODObjectInCellEvent().Remove(Handle_OnForEachHLODObjectInCell);
			}
			Handle_OnForEachHLODObjectInCell.Reset();
		}
	}

	Super::Deinitialize();
}

void UFastGeoWorldSubsystem::OnLevelStreamingStateChanged(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState PrevState, ELevelStreamingState NewState)
{
	if (World != GetWorld())
	{
		return;
	}

	if (LevelIfLoaded && LevelIfLoaded->GetWorld() && ((NewState == ELevelStreamingState::LoadedNotVisible) || (NewState == ELevelStreamingState::LoadedVisible)))
	{
		if (UFastGeoContainer* FastGeo = LevelIfLoaded->GetAssetUserData<UFastGeoContainer>())
		{
			FastGeo->PrecachePSOs();
		}
	}
}

void UFastGeoWorldSubsystem::OnLevelStartedAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* Level)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UFastGeoContainer* FastGeo = Level->GetAssetUserData<UFastGeoContainer>())
	{
		// Registration is handled by surrogate actor
		if (!FastGeo->IsUsingSurrogateComponents())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnLevelStartedAddToWorld);

			FastGeo->Register();
		}
	}
}

void UFastGeoWorldSubsystem::OnLevelStartedRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* Level)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UFastGeoContainer* FastGeo = Level->GetAssetUserData<UFastGeoContainer>())
	{
		// Unregistration is handled by surrogate actor
		if (!FastGeo->IsUsingSurrogateComponents())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnLevelStartedRemoveFromWorld);

			FastGeo->Unregister();
		}
	}
}

void UFastGeoWorldSubsystem::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	if (World == GetWorld())
	{
		DrainPendingRuntimeContainers();
	}
}

void UFastGeoWorldSubsystem::DrainPendingRuntimeContainers()
{
	check(IsInGameThread());

	static constexpr bool bWaitForCompletion = true;

	// Force-complete registering containers, then unregister them.
	// Note: Unregister() is safe to call even if an OnRegistered callback already called
	// DestroyRuntime -> Unregister; the state machine handles redundant calls as no-ops.
	while (!PendingRuntimeRegisterContainers.IsEmpty())
	{
		TSet<TObjectPtr<UFastGeoContainer>> Containers = MoveTemp(PendingRuntimeRegisterContainers);
		for (const TObjectPtr<UFastGeoContainer>& Container : Containers)
		{
			if (IsValid(Container))
			{
				Container->Tick(bWaitForCompletion);
				Container->Unregister();
				Container->Tick(bWaitForCompletion);
				check(Container->IsFullyUnregistered());
			}
		}
	}

	// Force-complete destroying containers.
	while (!PendingRuntimeDestroyContainers.IsEmpty())
	{
		TSet<TObjectPtr<UFastGeoContainer>> Containers = MoveTemp(PendingRuntimeDestroyContainers);
		for (const TObjectPtr<UFastGeoContainer>& Container : Containers)
		{
			if (IsValid(Container))
			{
				Container->Tick(bWaitForCompletion);
				check(Container->IsFullyUnregistered());
			}
		}
	}

	check(PendingRuntimeRegisterContainers.IsEmpty());
	check(PendingRuntimeDestroyContainers.IsEmpty());
}

void UFastGeoWorldSubsystem::OnGlobalComponentReregisterContextCreated()
{
	check(ReregisteringFastGeoContainers.IsEmpty());

	WaitForAllPendingWorkCompletion();

	FastGeo::FAtomicBoolGuard GuardWaitingForCompletion(bWaitingForCompletion, true);
	TGuardValue<bool> GuardIsReregistering(bIsReregistering, true);
	UWorld* World = GetWorld();
	for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		UFastGeoContainer* FastGeo = *It;
		if ((FastGeo->GetWorld() == World) && FastGeo->IsRegistered())
		{
			FastGeo->Unregister();
			ReregisteringFastGeoContainers.Add(FastGeo);
		}
	}

	WaitForAllPendingWorkCompletion();
}

void UFastGeoWorldSubsystem::OnGlobalComponentReregisterContextDestroyed()
{
	WaitForAllPendingWorkCompletion();

	FastGeo::FAtomicBoolGuard GuardWaitingForCompletion(bWaitingForCompletion, true);
	TGuardValue<bool> GuardIsReregistering(bIsReregistering, true);
	for (const TWeakObjectPtr<UFastGeoContainer>& WeakFastGeo : ReregisteringFastGeoContainers)
	{
		UFastGeoContainer* FastGeo = WeakFastGeo.Get();
		if (FastGeo && !FastGeo->IsRegistered())
		{
			FastGeo->Register();
		}
	}

	WaitForAllPendingWorkCompletion();

	ReregisteringFastGeoContainers.Empty();
}

#if WITH_EDITOR
void UFastGeoWorldSubsystem::OnPreRecreateScene(UWorld* World)
{
	if (World == GetWorld())
	{
		// Ensure all FastGeo work (sync + async) is fully drained before the scene is recreated.
		// In sync mode, render state work is advanced via UFastGeoContainer::Tick(),
		// not through the async job queue, so ProcessAsyncRenderStateJobs() alone is insufficient.
		// This guarantees no FastGeo render/physics jobs still reference the old scene.
		WaitForAllPendingWorkCompletion();
		check(!AsyncRenderStateJobQueue.IsValid() || AsyncRenderStateJobQueue->IsCompleted());

		// All callers of RecreateScene() are preceded by FGlobalComponentReregisterContext,
		// which unregisters all containers (resetting their AsyncContext). Validate this assumption.
#if DO_CHECK
		for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			ensureMsgf((*It)->GetWorld() != World || !(*It)->AsyncContext.IsInitialized(), TEXT("Container %s still has initialized AsyncContext in OnPreRecreateScene"), *GetNameSafe(*It));
		}
#endif
	}
}
#endif // WITH_EDITOR

void UFastGeoWorldSubsystem::OnLevelComponentsUpdated(UWorld* World, ULevel* Level)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UFastGeoContainer* FastGeo = Level->GetAssetUserData<UFastGeoContainer>())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnLevelComponentsUpdated);

		FastGeo->Register();

		static constexpr bool bWaitForCompletion = true;
		FastGeo->Tick(bWaitForCompletion);
	}
}

void UFastGeoWorldSubsystem::OnLevelComponentsCleared(UWorld* World, ULevel* Level)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UFastGeoContainer* FastGeo = Level->GetAssetUserData<UFastGeoContainer>())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnLevelComponentsCleared);

		if (World->IsBeingCleanedUp())
		{
			FastGeo->Unregister();

			static constexpr bool bWaitForCompletion = true;
			FastGeo->Tick(bWaitForCompletion);
		}
		else
		{
			check(FastGeo->IsFullyUnregistered());
		}
	}
}

void UFastGeoWorldSubsystem::OnAddLevelToWorldExtension(ULevel* InLevel, const bool bInWaitForCompletion, bool& bOutHasCompleted)
{
	FastGeo::FAtomicBoolGuard GuardValue(bWaitingForCompletion, bInWaitForCompletion);

	if (UFastGeoContainer* FastGeo = InLevel->GetAssetUserData<UFastGeoContainer>())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnAddLevelToWorldExtension);

		if (FastGeo->HasAnyPendingCreateTasks())
		{
			FastGeo->Tick(bInWaitForCompletion);
		}

		if (FastGeo->HasAnyPendingCreateTasks())
		{
			bOutHasCompleted = false;
		}
	}
}

void UFastGeoWorldSubsystem::OnRemoveLevelFromWorldExtension(ULevel* InLevel, const bool bInWaitForCompletion, bool& bOutHasCompleted)
{
	FastGeo::FAtomicBoolGuard GuardValue(bWaitingForCompletion, bInWaitForCompletion);

	if (UFastGeoContainer* FastGeo = InLevel->GetAssetUserData<UFastGeoContainer>())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnRemoveLevelFromWorldExtension);

		// Drain all pending work including destroy and PSO recreate (UseFallbackMaterialUntilPSOPrecached). 
		// Kept in sync with CheckNoPendingTasks which validates HasAnyPendingTasks() after removal completes.
		if (FastGeo->HasAnyPendingTasks())
		{
			FastGeo->Tick(bInWaitForCompletion);
		}

		if (FastGeo->HasAnyPendingTasks())
		{
			bOutHasCompleted = false;
		}
	}
}

void UFastGeoWorldSubsystem::OnUpdateLevelStreaming()
{
	if (GetWorld()->GetShouldForceUnloadStreamingLevels())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::OnUpdateLevelStreaming);
		WaitForAllPendingWorkCompletion();
	}
}

void UFastGeoWorldSubsystem::WaitForAllPendingWorkCompletion()
{
	check(IsInGameThread());
	FastGeo::FAtomicBoolGuard GuardWaitingForCompletion(bWaitingForCompletion, true);

	static constexpr bool bWaitForCompletion = true;

#if DO_CHECK
	int32 SafetyMaxIteration = 0;
#endif
	
	// Converge all FastGeo containers in this world to their target states
	UWorld* World = GetWorld();
	while (true)
	{
		for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UFastGeoContainer* FastGeo = *It;
			if (FastGeo->GetWorld() == World)
			{
				FastGeo->Tick(bWaitForCompletion);
			}
		}

		// Drain recreate queue
		ProcessPendingRecreate();
		if (PendingRecreateManager.IsEmpty())
		{
			break;
		}

#if DO_CHECK
		checkf(++SafetyMaxIteration < 1000, TEXT("WaitForAllPendingWorkCompletion() not converging."));
#endif
	}

	// The TObjectIterator loop above already ticked all containers to completion.
	// Clear the register set before draining -- don't force-unregister, as this may be called
	// during FGlobalComponentReregisterContext where containers must survive.
	PendingRuntimeRegisterContainers.Empty();
	DrainPendingRuntimeContainers();

	FPhysScene* PhysScene = World ? World->GetPhysicsScene() : nullptr;
	check(!PhysScene || !PhysScene->HasAsyncPhysicsStateJobs());
	check(!AsyncRenderStateJobQueue.IsValid() || AsyncRenderStateJobQueue->IsCompleted());
	check(PendingRecreateManager.IsEmpty());
	check(PendingRuntimeRegisterContainers.IsEmpty());
	check(PendingRuntimeDestroyContainers.IsEmpty());
}

void UFastGeoWorldSubsystem::AddToComponentsPendingRecreate(FFastGeoComponent* InComponent)
{
	PendingRecreateManager.Add(InComponent);
}

void UFastGeoWorldSubsystem::ProcessPendingRecreate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFastGeoWorldSubsystem::ProcessPendingRecreate);

	if (PendingRecreateManager.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	if (!PendingRecreateManager.GetComponents().IsEmpty())
	{
		ComponentsPreRecreateEvent.Broadcast(PendingRecreateManager.GetComponents());
	}
#endif

	PendingRecreateManager.Process(GetWorld());
}

bool UFastGeoWorldSubsystem::FFastGeoPendingRecreateManager::IsEmpty() const
{
	if (!Components.IsEmpty())
	{
		return false;
	}
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (!PSORecreateBatch.IsCompleted())
	{
		return false;
	}
#endif
	return true;
}

void UFastGeoWorldSubsystem::FFastGeoPendingRecreateManager::Reset()
{
	Components.Reset();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	PSORecreateBatch.Reset();
#endif
	bHasNonPSORecreate = false;
}

void UFastGeoWorldSubsystem::FFastGeoPendingRecreateManager::Add(FFastGeoComponent* Component)
{
	Components.Emplace(Component);
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (!Component->NeedsPSORecreate())
#endif
	{
		bHasNonPSORecreate = true;
	}
}

void UFastGeoWorldSubsystem::FFastGeoPendingRecreateManager::Process(UWorld* World)
{
	CSV_CUSTOM_STAT(FastGeo, PendingRecreate, Components.Num(), ECsvCustomStatOp::Set);

	// Pass 1: Process non-PSO recreates immediately (preserve same-frame guarantee).
	// Skipped when only PSO recreates are pending (common case during PSO completion bursts).
	if (bHasNonPSORecreate)
	{
		bHasNonPSORecreate = false;
		for (int32 i = Components.Num() - 1; i >= 0; --i)
		{
			if (FFastGeoComponent* Component = Components[i].TryGetRegistered<FFastGeoComponent>())
			{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
				if (Component->NeedsPSORecreate())
				{
					continue;
			}
#endif
			Component->DestroyRenderState(nullptr);
			Component->CreateRenderState(nullptr);
		}
			Components.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Pass 2: PSO recreates with ParallelFor + time budget.
	// Uses TFastGeoRenderStateBatch<FFastGeoRegisteredComponent> to persist across frames
	// without raw pointers. Stale components (container unregistered between frames) are
	// validated inline via TryGetRegistered and skipped at zero cost.
	if (!Components.IsEmpty())
	{
		PSORecreateBatch.ComponentsToProcess.Append(MoveTemp(Components));
		PSORecreateBatch.NumToProcess = PSORecreateBatch.ComponentsToProcess.Num() - PSORecreateBatch.TotalNumProcessed;
	}

	if (!PSORecreateBatch.IsCompleted())
	{
		// Bypass budget when draining all pending work to avoid the safety iteration limit.
		const bool bForceComplete = World && World->GetSubsystemChecked<UFastGeoWorldSubsystem>()->IsWaitingForCompletion();
		const UE::FTimeout Timeout = (!bForceComplete && FastGeo::GSyncPSORecreateRenderStateTimeBudgetMS > 0.0f) ? UE::FTimeout(FastGeo::GSyncPSORecreateRenderStateTimeBudgetMS / 1000.0) : UE::FTimeout::Never();

		FastGeo::AdvanceRenderStateBudgeted(PSORecreateBatch, Timeout, [](FFastGeoRegisteredComponent& Ref)
		{
			if (FFastGeoComponent* Component = Ref.TryGetRegistered<FFastGeoComponent>())
			{
				Component->DestroyRenderState(nullptr);
				Component->CreateRenderState(nullptr);
			}
		});

		CSV_CUSTOM_STAT(FastGeo, PSORecreateBatchRemaining, PSORecreateBatch.ComponentsToProcess.Num() - PSORecreateBatch.TotalNumProcessed, ECsvCustomStatOp::Set);

		if (PSORecreateBatch.IsCompleted())
		{
			// Use Empty() instead of Reset() to free memory from large PSO bursts.
			PSORecreateBatch.ComponentsToProcess.Empty();
			PSORecreateBatch.Reset();
		}
	}
#endif
}

void UFastGeoWorldSubsystem::ForEachHLODObjectInCell(const UWorldPartitionRuntimeCell* InCell, TFunction<void(IWorldPartitionHLODObject*)> InFunc)
{
	check(InCell);
	check(InCell->GetLevel());

	if (UFastGeoContainer* FastGeo = InCell->GetLevel()->GetAssetUserData<UFastGeoContainer>())
	{
		// Iterate over clusters in the container, and call InFunc on all HLOD objects
		FastGeo->ForEachComponentCluster<FFastGeoHLOD>([&InFunc](FFastGeoHLOD& HLOD)
		{
			InFunc(&HLOD);
		});
	}
}

#if DO_CHECK
void UFastGeoWorldSubsystem::CheckNoPendingTasks(ULevel* Level, UWorld* World, bool bIncludeRecreateTasks)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UFastGeoContainer* FastGeo = Level->GetAssetUserData<UFastGeoContainer>())
	{
		// bIncludeRecreateTasks=false: recreate (UseFallbackMaterialUntilPSOPrecached) is
		// post-registration background work that can legitimately be pending after AddToWorld.
		// bIncludeRecreateTasks=true: after RemoveFromWorld, all work including recreate must be done.
		const bool bHasPendingTasks = bIncludeRecreateTasks ? FastGeo->HasAnyPendingTasks() : FastGeo->HasAnyPendingRegistrationTasks();
		check(!bHasPendingTasks);
	}
}

void UFastGeoWorldSubsystem::OnLevelAddedToWorld(ULevel* Level, UWorld* World)
{
	check(Level);
	// Recreate can be pending after AddToWorld -- it's post-registration background work.
	CheckNoPendingTasks(Level, World, /*bIncludeRecreateTasks=*/ false);
}

void UFastGeoWorldSubsystem::OnLevelRemovedFromWorld(ULevel* Level, UWorld* World)
{
	// Null means every sublevel is being removed
	if (!Level)
	{
		check(World);
		for (ULevel* SubLevel : World->GetLevels())
		{
			CheckNoPendingTasks(SubLevel, World);
		}
	}
	else
	{
		CheckNoPendingTasks(Level, World);
	}
}
#endif

bool UFastGeoWorldSubsystem::IsWaitingForCompletion() const
{
	// FastGeo's own GT-side wait windows stamp bWaitingForCompletion via FAtomicBoolGuard;
	// acquire-load is safe from any thread including generic workers.
	if (bWaitingForCompletion.load(std::memory_order_acquire))
	{
		return true;
	}

	const UWorld* World = GetWorld();
	return World->GetIsInBlockTillLevelStreamingCompleted() ||
			World->GetShouldForceUnloadStreamingLevels() ||
			World->IsBeingCleanedUp();
}

bool UFastGeoWorldSubsystem::IsTickableInEditor() const
{
	return true;
}

void UFastGeoWorldSubsystem::Tick(float DeltaTime)
{
	{
		FWriteScopeLock ScopeLock(Lock);
		++TimeEpoch;
		UsedAsyncRenderStateTasksTimeBudgetMS = 0;
		UsedNumComponentsToProcessBudget = 0;
	}

	// Tick runtime containers and remove completed ones.
	auto TickAndDrainCompleted = [](TSet<TObjectPtr<UFastGeoContainer>>& Containers, auto IsCompletedFunc)
	{
		for (auto It = Containers.CreateIterator(); It; ++It)
		{
			UFastGeoContainer* Container = *It;
			bool bShouldRemove = !IsValid(Container) || IsCompletedFunc(Container);
			if (!bShouldRemove)
			{
				Container->Tick();
				bShouldRemove = IsCompletedFunc(Container);
			}
			if (bShouldRemove)
			{
				It.RemoveCurrent();
			}
		}
	};

	// Level-streamed containers are driven by OnAddLevelToWorldExtension/OnRemoveLevelFromWorldExtension instead.
	// Also evict containers marked for destruction (e.g., DestroyRuntime called from an OnRegistered callback).
	// These are tracked by PendingRuntimeDestroyContainers and should not be ticked from both sets.
	// Keep containers in the set until registration AND post-registration recreate
	// (UseFallbackMaterialUntilPSOPrecached) complete. Without this, a standalone runtime
	// container would be drained after registration, and nobody would tick the async queue
	// to process the recreate job's End_GameThread callback.
	TickAndDrainCompleted(PendingRuntimeRegisterContainers, [](const UFastGeoContainer* Container)
	{
		bool bRegistrationComplete = Container->IsFullyRegistered();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
		bRegistrationComplete = bRegistrationComplete && !Container->HasAnyPendingRecreateTasks();
#endif
		return bRegistrationComplete || Container->bIsBeingAsyncDestroyed;
	});

	// Containers complete destruction when fully unregistered.
	TickAndDrainCompleted(PendingRuntimeDestroyContainers, [](const UFastGeoContainer* Container) { return Container->IsFullyUnregistered(); });

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Safety net: poll containers with PSO-deferred components to detect lost callbacks.
	// PSO callbacks can be lost if a component is destroyed before the callback fires, or if
	// the FPSOPrecacheFinishedTask is never dispatched (lifecycle ID mismatch). Without this
	// poll, a lost callback would leave the container parked forever with AddToWorld never
	// completing. The cost is negligible: only containers with active deferred work are polled,
	// and the check is a simple IsPSOPrecaching() per deferred component.
	for (auto It = ContainersWithPSODeferredWork.CreateIterator(); It; ++It)
	{
		UFastGeoContainer* Container = (*It).Get();
		if (!IsValid(Container) || !Container->HasPSODeferredComponents())
		{
			It.RemoveCurrent();
		}
		else
		{
			Container->CheckPSODeferredReadiness(/*bPollPending=*/ true);
		}
	}

	// Drive async recreate jobs that were pushed after OnAddLevelToWorldExtension completed.
	// Level-streamed containers are registered and created during AddToWorld, which processes
	// the async job queue to completion via OnAddLevelToWorldExtension. However, PSO callbacks
	// (OnComponentPSOPrecacheCompleted) can fire after the extension finishes, pushing recreate
	// jobs to the queue with no one left to tick them. Runtime containers don't have this
	// problem because they stay in PendingRuntimeRegisterContainers and get Container->Tick()
	// each frame. Without this, recreate jobs for level-streamed containers would stall until
	// the next level streaming event (e.g., camera move streaming in new levels).
	if (!ContainersWithAsyncPSORecreateWork.IsEmpty())
	{
		for (auto It = ContainersWithAsyncPSORecreateWork.CreateIterator(); It; ++It)
		{
			if (!IsValid((*It).Get()) || !(*It)->HasAnyPendingRecreateTasks())
			{
				It.RemoveCurrent();
			}
		}

		if (!ContainersWithAsyncPSORecreateWork.IsEmpty())
		{
			ProcessAsyncRenderStateJobs();
		}
	}
#endif

#if !UE_BUILD_SHIPPING
	if (FastGeo::GDebugTraceHUD)
	{
		TickDebugTraceHUD();
	}
#endif
}

TStatId UFastGeoWorldSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UFastGeoWorldSubsystem, STATGROUP_Tickables);
}

void UFastGeoWorldSubsystem::RequestAsyncRenderStateTasksBudget_Concurrent(float& OutAvailableTimeBudgetMS, int32& OutAvailableComponentsBudget, int32& OutTimeEpoch)
{
	FWriteScopeLock ScopeLock(Lock);

	bool bUnlimitedBudget = IsInGameThread() ? IsWaitingForCompletion() : false;

	if (bUnlimitedBudget || FastGeo::GAsyncRenderStateTaskTimeBudgetMS == 0)
	{
		OutAvailableTimeBudgetMS = FLT_MAX;
	}
	else
	{
		OutAvailableTimeBudgetMS = FMath::Max(FastGeo::GAsyncRenderStateTaskTimeBudgetMS - UsedAsyncRenderStateTasksTimeBudgetMS, 0);
	}

	if (bUnlimitedBudget || FastGeo::GAsyncRenderStateTaskMaxNumComponentsToProcess == 0)
	{
		OutAvailableComponentsBudget = INT32_MAX;
	}
	else
	{
		OutAvailableComponentsBudget = FMath::Max(FastGeo::GAsyncRenderStateTaskMaxNumComponentsToProcess - UsedNumComponentsToProcessBudget, 0);
	}

	OutTimeEpoch = TimeEpoch;
}

void UFastGeoWorldSubsystem::CommitAsyncRenderStateTasksBudget_Concurrent(float InUsedTimeBudgetMS, int32& InUsedComponentsBudget, int32 InTimeEpoch)
{
	FWriteScopeLock ScopeLock(Lock);

	if (InTimeEpoch == TimeEpoch)
	{
		if (FastGeo::GAsyncRenderStateTaskTimeBudgetMS != 0)
		{
			UsedAsyncRenderStateTasksTimeBudgetMS += InUsedTimeBudgetMS;
		}

		if (FastGeo::GAsyncRenderStateTaskMaxNumComponentsToProcess != 0)
		{
			UsedNumComponentsToProcessBudget += InUsedComponentsBudget;
		}
	}
}

void UFastGeoWorldSubsystem::PushRuntimeContainerForRegistration(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->IsRuntime());
	check(!FastGeo->bIsBeingAsyncDestroyed);

	if (!FastGeo->IsFullyRegistered())
	{
		PendingRuntimeRegisterContainers.Add(FastGeo);
	}
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void UFastGeoWorldSubsystem::RegisterContainerWithPSODeferredWork(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	ContainersWithPSODeferredWork.Add(FastGeo);
}

void UFastGeoWorldSubsystem::UnregisterContainerWithPSODeferredWork(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	ContainersWithPSODeferredWork.Remove(FastGeo);
}

void UFastGeoWorldSubsystem::RegisterContainerWithAsyncPSORecreateWork(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	ContainersWithAsyncPSORecreateWork.Add(FastGeo);
}

void UFastGeoWorldSubsystem::UnregisterContainerWithAsyncPSORecreateWork(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	ContainersWithAsyncPSORecreateWork.Remove(FastGeo);
}
#endif

void UFastGeoWorldSubsystem::PushRuntimeContainerForDestruction(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->IsRuntime());
	check(FastGeo->bIsBeingAsyncDestroyed);

	// Don't remove from PendingRuntimeRegisterContainers here -- the register
	// tick loop checks bIsBeingAsyncDestroyed and removes it. This avoids
	// invalidating iterators if DestroyRuntime is called from an OnRegistered callback.

	if (!FastGeo->IsFullyUnregistered())
	{
		PendingRuntimeDestroyContainers.Add(FastGeo);
	}
}

FFastGeoAsyncRenderStateJobQueue* UFastGeoWorldSubsystem::GetOrCreateJobQueue()
{
	if (!AsyncRenderStateJobQueue.IsValid())
	{
		AsyncRenderStateJobQueue = MakeUnique<FFastGeoAsyncRenderStateJobQueue>(GetWorld()->Scene);
	}
	return AsyncRenderStateJobQueue.Get();
}

void UFastGeoWorldSubsystem::PushAsyncPrecachePSOsJob(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->GetWorld() == GetWorld());
	GetOrCreateJobQueue()->AddJob({FastGeo, FFastGeoAsyncRenderStateJobQueue::EJobType::PrecachePSOs});
}

void UFastGeoWorldSubsystem::PushAsyncCreateRenderStateJob(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->GetWorld() == GetWorld());

	static bool bHasWarnedAboutSSAM = false;
	if (!bHasWarnedAboutSSAM && !FSimpleStreamableAssetManager::IsEnabled())
	{
		bHasWarnedAboutSSAM = true;
		UE_LOGF(LogFastGeoStreaming, Warning, "FSimpleStreamableAssetManager must be enabled for FastGeo. Set s.StreamableAssets.UseSimpleStreamableAssetManager=1");
	}

	GetOrCreateJobQueue()->AddJob({ FastGeo, FFastGeoAsyncRenderStateJobQueue::EJobType::CreateRenderState });
}

void UFastGeoWorldSubsystem::PushAsyncDestroyRenderStateJob(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->GetWorld() == GetWorld());
	GetOrCreateJobQueue()->AddJob({ FastGeo, FFastGeoAsyncRenderStateJobQueue::EJobType::DestroyRenderState });
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void UFastGeoWorldSubsystem::PushAsyncDeferredCreateJob(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->GetWorld() == GetWorld());
	GetOrCreateJobQueue()->AddJob({ FastGeo, FFastGeoAsyncRenderStateJobQueue::EJobType::DeferredCreateRenderState });
}

void UFastGeoWorldSubsystem::PushAsyncRecreateRenderStateJob(UFastGeoContainer* FastGeo)
{
	check(IsInGameThread());
	check(FastGeo->GetWorld() == GetWorld());
	GetOrCreateJobQueue()->AddJob({ FastGeo, FFastGeoAsyncRenderStateJobQueue::EJobType::RecreateRenderState });
}
#endif

void UFastGeoWorldSubsystem::ProcessAsyncRenderStateJobs(bool bWaitForCompletion)
{
	check(IsInGameThread());
	if (AsyncRenderStateJobQueue.IsValid())
	{
		AsyncRenderStateJobQueue->Tick(bWaitForCompletion);
		if (AsyncRenderStateJobQueue->IsCompleted())
		{
			AsyncRenderStateJobQueue.Reset();
		}
	}
}

void UFastGeoWorldSubsystem::PushAsyncCreatePhysicsStateJobs(UFastGeoContainer* FastGeo)
{
	FastGeo->OnCreatePhysicsStateBegin_GameThread();
}

void UFastGeoWorldSubsystem::PushAsyncDestroyPhysicsStateJobs(UFastGeoContainer* FastGeo)
{
	FastGeo->OnDestroyPhysicsStateBegin_GameThread();
}

#if !UE_BUILD_SHIPPING
void UFastGeoWorldSubsystem::TickDebugTraceHUD()
{
	UWorld* World = GetWorld();
	APlayerController* PC = World ? World->GetFirstPlayerController() : nullptr;
	if (!PC || !GEngine)
	{
		return;
	}

	auto DebugPrint = [](const FColor& Color, const FString& Msg)
	{
		GEngine->AddOnScreenDebugMessage(-1, 0.0f, Color, Msg, false);
	};

	FVector ViewLocation;
	FRotator ViewRotation;
	PC->GetPlayerViewPoint(ViewLocation, ViewRotation);
	constexpr float TraceDistance = 50000.0f;
	const FVector TraceEnd = ViewLocation + ViewRotation.Vector() * TraceDistance;

	FHitResult HitResult;
	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;
	QueryParams.bReturnPhysicalMaterial = true;
	if (!World->LineTraceSingleByChannel(HitResult, ViewLocation, TraceEnd, ECC_Visibility, QueryParams))
	{
		DebugPrint(FColor::Yellow, TEXT("[FastGeo Trace] No hit"));
		return;
	}

	DrawDebugSphere(World, HitResult.ImpactPoint, 25.0f, 12, FColor::Green, false, 0.0f);

	const FFastGeoPhysicsBodyInstanceOwner* FastGeoOwner = FFastGeoPhysicsBodyInstanceOwner::FromHitResult(HitResult);
	if (!FastGeoOwner)
	{
		UPrimitiveComponent* HitComp = HitResult.GetComponent();
		AActor* HitActor = HitComp ? HitComp->GetOwner() : nullptr;
		DebugPrint(FColor::Yellow, FString::Printf(TEXT("[FastGeo Trace] Not FastGeo -- Actor: %s  Component: %s"), *GetNameSafe(HitActor), *GetNameSafe(HitComp)));
		return;
	}
	FFastGeoComponent* FastGeoComponent = FastGeoOwner->GetOwnerComponent();
	if (!FastGeoComponent)
	{
		DebugPrint(FColor::Red, TEXT("[FastGeo Trace] Null FastGeo component"));
		return;
	}

	UFastGeoContainer* FastGeoContainer = FastGeoOwner->GetOwnerContainer();
	if (!FastGeoContainer)
	{
		DebugPrint(FColor::Red, TEXT("[FastGeo Trace] Null FastGeo container"));
		return;
	}

	auto GetComponentType = [](FFastGeoComponent& Component) -> const TCHAR*
	{
		if (Component.IsA<FFastGeoInstancedStaticMeshComponent>())
		{
			return TEXT("FastGeoInstancedStaticMesh");
		}
		if (Component.IsA<FFastGeoStaticMeshComponentBase>())
		{
			return TEXT("FastGeoStaticMesh");
		}
		return TEXT("FastGeoComponent");
	};

	auto GetComponentStaticMesh = [](FFastGeoComponent& Component) -> FString
	{
		if (FFastGeoStaticMeshComponentBase* SM = Component.CastTo<FFastGeoStaticMeshComponentBase>())
		{
			if (UStaticMesh* Mesh = SM->GetStaticMesh())
			{
				return Mesh->GetName();
			}
		}
		return TEXT("None");
	};

	auto GetComponentSurrogateName = [](FFastGeoComponent& Component) -> FString
	{
		if (FFastGeoPrimitiveComponent* PrimComp = Component.CastTo<FFastGeoPrimitiveComponent>())
		{
			if (UFastGeoSurrogateComponent* Surrogate = PrimComp->GetSurrogateComponent())
			{
				if (AActor* Owner = Surrogate->GetOwner())
				{
					return FString::Printf(TEXT("%s.%s"), *Owner->GetName(), *Surrogate->GetName());
				}
				return Surrogate->GetName();
			}
		}
		return TEXT("None");
	};

	// Builds a compact summary: "Block: Ch1, Ch2 | Overlap: Ch3"
	auto FormatCollisionResponses = [](const FCollisionResponseContainer& Responses) -> FString
	{
		UEnum* ChannelEnum = StaticEnum<ECollisionChannel>();
		FString BlockChannels, OverlapChannels;
		for (int32 i = 0; i < ECC_MAX; ++i)
		{
			ECollisionResponse Response = Responses.GetResponse((ECollisionChannel)i);
			if (Response == ECR_Block)
			{
				if (BlockChannels.Len() > 0)
				{
					BlockChannels += TEXT(", ");
				}
				BlockChannels += ChannelEnum->GetNameStringByValue(i);
			}
			else if (Response == ECR_Overlap)
			{
				if (OverlapChannels.Len() > 0)
				{
					OverlapChannels += TEXT(", ");
				}
				OverlapChannels += ChannelEnum->GetNameStringByValue(i);
			}
		}

		constexpr int32 MaxResponseLen = 200;
		if (BlockChannels.Len() > MaxResponseLen)
		{
			BlockChannels.LeftInline(MaxResponseLen);
			BlockChannels += TEXT("...");
		}
		if (OverlapChannels.Len() > MaxResponseLen)
		{
			OverlapChannels.LeftInline(MaxResponseLen);
			OverlapChannels += TEXT("...");
		}

		FString Result;
		if (BlockChannels.Len() > 0)
		{
			Result += FString::Printf(TEXT("Block: %s"), *BlockChannels);
		}
		if (OverlapChannels.Len() > 0)
		{
			if (Result.Len() > 0)
			{
				Result += TEXT(" | ");
			}
			Result += FString::Printf(TEXT("Overlap: %s"), *OverlapChannels);
		}
		return Result.Len() > 0 ? Result : TEXT("(all Ignore)");
	};

	// Helper to get enum display name for ECollisionEnabled::Type
	auto GetCollisionEnabledName = [](ECollisionEnabled::Type CollisionEnabled) -> FString
	{
		return StaticEnum<ECollisionEnabled::Type>()->GetNameStringByValue(static_cast<__underlying_type(ECollisionEnabled::Type)>(CollisionEnabled));
	};

	// Helper to get enum display name for ECollisionChannel
	auto GetCollisionChannelName = [](ECollisionChannel Channel) -> FString
	{
		return StaticEnum<ECollisionChannel>()->GetNameStringByValue(Channel);
	};

	// Color scheme:
	// White      - top-level header
	// Purple     - hit result details (impact, normal, phys material)
	// Hot pink   - level section
	// Emerald    - surrogate section
	// Orange     - physics sub-sections (collision, responses, walkable)
	// Red        - mismatches / errors
	// Yellow     - IPhysicsBodyInstanceOwner section
	// Light blue - container section (header, labels, other components)
	// Cyan       - hit component data
	const FColor LevelColor(255, 100, 180);
	const FColor SurrogateColor(0, 200, 120);

	DebugPrint(FColor::White, TEXT("[FastGeo Trace]"));

	// Hit result details
	DebugPrint(FColor::Purple, TEXT("  [HitResult]"));
	DebugPrint(FColor::Purple, FString::Printf(TEXT("    ImpactPoint: %s"), *HitResult.ImpactPoint.ToString()));
	DebugPrint(FColor::Purple, FString::Printf(TEXT("    ImpactNormal: %s"), *HitResult.ImpactNormal.ToString()));
	DebugPrint(FColor::Purple, FString::Printf(TEXT("    PhysicalMaterial: %s"), HitResult.PhysMaterial.IsValid() ? *HitResult.PhysMaterial->GetName() : TEXT("None")));
	DebugPrint(FColor::Purple, FString::Printf(TEXT("    Distance: %.1f"), HitResult.Distance));

	// Level / Cell
	const ULevel* Level = FastGeoContainer->GetLevel();
	if (Level)
	{
		DebugPrint(LevelColor, TEXT("  [Level]"));
		UWorld* LevelWorld = Level->GetWorld();
		DebugPrint(LevelColor, FString::Printf(TEXT("    Name: %s.%s"), LevelWorld ? *LevelWorld->GetName() : TEXT("None"), *Level->GetName()));

		if (const IWorldPartitionCell* Cell = Level->GetWorldPartitionRuntimeCell())
		{
			DebugPrint(LevelColor, FString::Printf(TEXT("    Cell: %s (%s)"), *Cell->GetDebugName(), *Cell->GetLevelPackageName().ToString()));
		}
	}

	// Capture surrogate values (if surrogate was hit) for comparison against the owner physics properties below.
	bool bHasSurrogate = false;
	UBodySetup* SurrogateBodySetup = nullptr;
	ECollisionEnabled::Type SurrogateCollisionEnabled = ECollisionEnabled::NoCollision;
	ECollisionChannel SurrogateObjectType = ECC_WorldStatic;
	FName SurrogateCollisionProfile;
	FCollisionResponseContainer SurrogateResponses;
	Chaos::FPhysicsObject* SurrogatePhysicsObject = nullptr;
	FBodyInstance* SurrogateBodyInstance = nullptr;
	FWalkableSlopeOverride SurrogateWalkableSlope;

	// Surrogate section (only when a surrogate component was hit)
	if (const UFastGeoSurrogateComponent* SurrogateComponent = Cast<UFastGeoSurrogateComponent>(HitResult.GetComponent()))
	{
		FString SurrogateName = SurrogateComponent->GetName();
		if (AActor* SurrogateActor = SurrogateComponent->GetOwner())
		{
			SurrogateName = FString::Printf(TEXT("%s.%s"), *SurrogateActor->GetName(), *SurrogateName);
		}
		DebugPrint(SurrogateColor, TEXT("  [Surrogate]"));
		DebugPrint(SurrogateColor, FString::Printf(TEXT("    Name: %s"), *SurrogateName));
		DebugPrint(SurrogateColor, FString::Printf(TEXT("    Transform: %s"), *SurrogateComponent->GetComponentTransform().ToString()));
		DebugPrint(SurrogateColor, FString::Printf(TEXT("    BodyCount: %d"), SurrogateComponent->GetAllPhysicsObjects().Num()));

		if (FBodyInstance* BodyInstance = SurrogateComponent->GetBodyInstance(NAME_None, true, HitResult.Item))
		{
			const char* DebugName = "None";
#if USE_BODYINSTANCE_DEBUG_NAMES
			if (BodyInstance->CharDebugName.IsValid())
			{
				DebugName = BodyInstance->CharDebugName->GetData();
			}
#endif
			const int32 DecodedIndex = FFastGeoSurrogateBodyInstanceIndex::Decode(HitResult.Item);
			DebugPrint(SurrogateColor, FString::Printf(TEXT("    BodyInstance (0x%08X -> idx:%d): %hs"), HitResult.Item, DecodedIndex, DebugName));

			bHasSurrogate = true;
			SurrogateBodyInstance = BodyInstance;
			SurrogateBodySetup = BodyInstance->GetBodySetup();
			SurrogateCollisionEnabled = SurrogateComponent->GetCollisionEnabled();
			SurrogateObjectType = SurrogateComponent->GetCollisionObjectType();
			SurrogateCollisionProfile = BodyInstance->GetCollisionProfileName();
			SurrogateResponses = BodyInstance->GetResponseToChannels();
			SurrogatePhysicsObject = SurrogateComponent->GetPhysicsObjectById(HitResult.Item);
			SurrogateWalkableSlope = SurrogateComponent->GetWalkableSlopeOverride();

			// Surrogate physics properties
			DebugPrint(FColor::Orange, TEXT("    [Physics]"));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      BodySetup: %s (%p)"), SurrogateBodySetup ? *SurrogateBodySetup->GetName() : TEXT("None"), SurrogateBodySetup));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      CollisionEnabled: %s"), *GetCollisionEnabledName(SurrogateCollisionEnabled)));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      ObjectType: %s"), *GetCollisionChannelName(SurrogateObjectType)));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      CollisionProfile: %s"), *SurrogateCollisionProfile.ToString()));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      Responses: %s"), *FormatCollisionResponses(SurrogateResponses)));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      PhysicsObject: %p"), SurrogatePhysicsObject));

			// Character movement properties
			const TCHAR* StepUpStr = (SurrogateComponent->CanCharacterStepUpOn == ECB_Yes) ? TEXT("Yes") : (SurrogateComponent->CanCharacterStepUpOn == ECB_No) ? TEXT("No") : TEXT("Owner");
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      CanCharacterStepUpOn: %s"), StepUpStr));

			const FWalkableSlopeOverride& SlopeOverride = BodyInstance->GetWalkableSlopeOverride();
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      WalkableSlope: %s (Angle: %.1f)"), *StaticEnum<EWalkableSlopeBehavior>()->GetNameStringByValue(SlopeOverride.GetWalkableSlopeBehavior()), SlopeOverride.GetWalkableSlopeAngle()));
		}
	}

	// IPhysicsBodyInstanceOwner properties
	DebugPrint(FColor::Yellow, TEXT("  [BodyInstanceOwner]"));
	DebugPrint(FColor::Yellow, FString::Printf(TEXT("    Transform: %s"), *FastGeoOwner->GetPhysicsOwnerTransform().ToString()));
	DebugPrint(FColor::Yellow, FString::Printf(TEXT("    IsWorldGeometry: %s"), FastGeoOwner->IsPhysicsObjectWorldGeometry() ? TEXT("true") : TEXT("false")));
	DebugPrint(FColor::Yellow, FString::Printf(TEXT("    IsStaticPhysics: %s"), FastGeoOwner->IsStaticPhysics() ? TEXT("true") : TEXT("false")));

	// Physics sub-section. When a surrogate was hit, only show BodySetup + any mismatched properties (in red);
	// otherwise show the full set of owner physics properties.
	DebugPrint(FColor::Orange, TEXT("    [Physics]"));
	UBodySetup* OwnerBodySetup = FastGeoOwner->GetPhysicsBodySetup();
	DebugPrint(FColor::Orange, FString::Printf(TEXT("      BodySetup: %s (%p)"), OwnerBodySetup ? *OwnerBodySetup->GetName() : TEXT("None"), OwnerBodySetup));

	ECollisionEnabled::Type OwnerCollisionEnabled = FastGeoOwner->GetCollisionEnabled();
	ECollisionChannel OwnerObjectType = FastGeoOwner->GetCollisionObjectType();
	FBodyInstance* OwnerBodyInstance = FastGeoOwner->GetBodyInstance();

	if (bHasSurrogate)
	{
		// HitResult.PhysicsObject is populated by the engine collision conversion via the IPhysicsBodyInstanceOwner path.
		// The surrogate's GetPhysicsObjectById(Hit.Item) override must resolve to the same per-item Chaos handle.
		if (SurrogatePhysicsObject != HitResult.PhysicsObject)
		{
			DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH PhysicsObject: Surrogate=%p vs HitResult=%p"), SurrogatePhysicsObject, HitResult.PhysicsObject));
		}
		// SurrogateComponent->GetWalkableSlopeOverride() reads the placeholder BodyInstance, which is seeded from the descriptor in
		// SetSurrogateComponentDescriptor. The per-item BodyInstance carries the canonical value. A divergence means the placeholder
		// was not seeded (or seeded with the wrong value).
		if (SurrogateBodyInstance)
		{
			const FWalkableSlopeOverride& PerItemWalkableSlope = SurrogateBodyInstance->GetWalkableSlopeOverride();
			if (SurrogateWalkableSlope.GetWalkableSlopeBehavior() != PerItemWalkableSlope.GetWalkableSlopeBehavior()
				|| SurrogateWalkableSlope.GetWalkableSlopeAngle() != PerItemWalkableSlope.GetWalkableSlopeAngle())
			{
				UEnum* BehaviorEnum = StaticEnum<EWalkableSlopeBehavior>();
				DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH WalkableSlope: Surrogate=%s/%.1f vs PerItem=%s/%.1f"),
					*BehaviorEnum->GetNameStringByValue(SurrogateWalkableSlope.GetWalkableSlopeBehavior()),
					SurrogateWalkableSlope.GetWalkableSlopeAngle(),
					*BehaviorEnum->GetNameStringByValue(PerItemWalkableSlope.GetWalkableSlopeBehavior()),
					PerItemWalkableSlope.GetWalkableSlopeAngle()));
			}
		}
		if (SurrogateBodySetup != OwnerBodySetup)
		{
			DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH BodySetup: Surrogate=%s (%p) vs Owner=%s (%p)"),
				SurrogateBodySetup ? *SurrogateBodySetup->GetName() : TEXT("None"), SurrogateBodySetup,
				OwnerBodySetup ? *OwnerBodySetup->GetName() : TEXT("None"), OwnerBodySetup));
		}
		if (SurrogateCollisionEnabled != OwnerCollisionEnabled)
		{
			DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH CollisionEnabled: Surrogate=%s vs Owner=%s"), *GetCollisionEnabledName(SurrogateCollisionEnabled), *GetCollisionEnabledName(OwnerCollisionEnabled)));
		}
		if (SurrogateObjectType != OwnerObjectType)
		{
			DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH ObjectType: Surrogate=%s vs Owner=%s"), *GetCollisionChannelName(SurrogateObjectType), *GetCollisionChannelName(OwnerObjectType)));
		}
		if (OwnerBodyInstance)
		{
			const FName OwnerCollisionProfile = OwnerBodyInstance->GetCollisionProfileName();
			if (SurrogateCollisionProfile != OwnerCollisionProfile)
			{
				DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH CollisionProfile: Surrogate=%s vs Owner=%s"), *SurrogateCollisionProfile.ToString(), *OwnerCollisionProfile.ToString()));
			}

			const FCollisionResponseContainer& OwnerResponses = OwnerBodyInstance->GetResponseToChannels();
			UEnum* ChannelEnum = StaticEnum<ECollisionChannel>();
			UEnum* ResponseEnum = StaticEnum<ECollisionResponse>();
			TArray<FString> DivergentChannels;
			for (int32 i = 0; i < ECC_MAX; ++i)
			{
				// Skip deprecated/hidden channels: their response slots are not reliably initialized
				// and produce noise (e.g. ECC_OverlapAll_Deprecated).
				const FString ChannelName = ChannelEnum->GetNameStringByValue(i);
				if (ChannelName.IsEmpty() || ChannelName.Contains(TEXT("Deprecated"))
#if WITH_EDITOR
					|| ChannelEnum->HasMetaData(TEXT("Hidden"), ChannelEnum->GetIndexByValue(i))
#endif
				)
				{
					continue;
				}

				const ECollisionResponse S = SurrogateResponses.GetResponse((ECollisionChannel)i);
				const ECollisionResponse O = OwnerResponses.GetResponse((ECollisionChannel)i);
				if (S != O)
				{
					DivergentChannels.Add(FString::Printf(TEXT("%s[S:%s,O:%s]"),
						*ChannelName,
						*ResponseEnum->GetNameStringByValue(S),
						*ResponseEnum->GetNameStringByValue(O)));
				}
			}
			if (DivergentChannels.Num() > 0)
			{
				DebugPrint(FColor::Red, FString::Printf(TEXT("      MISMATCH Responses (%d channel%s): %s"),
					DivergentChannels.Num(),
					DivergentChannels.Num() == 1 ? TEXT("") : TEXT("s"),
					*FString::Join(DivergentChannels, TEXT(", "))));
			}
		}
	}
	else
	{
		// No surrogate was hit: show full owner physics properties.
		DebugPrint(FColor::Orange, FString::Printf(TEXT("      CollisionEnabled: %s"), *GetCollisionEnabledName(OwnerCollisionEnabled)));
		DebugPrint(FColor::Orange, FString::Printf(TEXT("      ObjectType: %s"), *GetCollisionChannelName(OwnerObjectType)));
		if (OwnerBodyInstance)
		{
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      CollisionProfile: %s"), *OwnerBodyInstance->GetCollisionProfileName().ToString()));
			DebugPrint(FColor::Orange, FString::Printf(TEXT("      Responses: %s"), *FormatCollisionResponses(OwnerBodyInstance->GetResponseToChannels())));
		}
	}

	// Container + Components
	const FColor ContainerColor(100, 200, 255);
	DebugPrint(ContainerColor, TEXT("  [Container]"));
	DebugPrint(ContainerColor, FString::Printf(TEXT("    Name: %s (%p)"), *FastGeoContainer->GetName(), FastGeoContainer));
	DebugPrint(ContainerColor, TEXT("    Hit Component:"));
	DebugPrint(FColor::Cyan, FString::Printf(TEXT("      %s (%p) Index=%d StaticMesh=%s Surrogate=%s"), GetComponentType(*FastGeoComponent), FastGeoComponent, FastGeoComponent->GetComponentIndex(), *GetComponentStaticMesh(*FastGeoComponent), *GetComponentSurrogateName(*FastGeoComponent)));

	// Count other components first to display the header with total count.
	int32 OtherCount = 0;
	FastGeoContainer->ForEachComponentCluster([&OtherCount, FastGeoComponent](const FFastGeoComponentCluster& Cluster)
	{
		Cluster.ForEachComponent<const FFastGeoComponent>([&OtherCount, FastGeoComponent](const FFastGeoComponent& Component)
		{
			if (Component.GetComponentIndex() != FastGeoComponent->GetComponentIndex())
			{
				++OtherCount;
			}
		});
	});

	if (OtherCount > 0)
	{
		DebugPrint(ContainerColor, FString::Printf(TEXT("    Other Components: (%d)"), OtherCount));

		constexpr int32 MaxOtherComponents = 50;
		int32 PrintedCount = 0;
		FastGeoContainer->ForEachComponentCluster([&](FFastGeoComponentCluster& Cluster)
		{
			Cluster.ForEachComponent<FFastGeoComponent>([&](FFastGeoComponent& Component)
			{
				if (Component.GetComponentIndex() != FastGeoComponent->GetComponentIndex() && PrintedCount < MaxOtherComponents)
				{
					++PrintedCount;
					DebugPrint(ContainerColor, FString::Printf(TEXT("      %s (%p) Index=%d StaticMesh=%s Surrogate=%s"), GetComponentType(Component), &Component, Component.GetComponentIndex(), *GetComponentStaticMesh(Component), *GetComponentSurrogateName(Component)));
				}
			});
		});

		if (OtherCount > MaxOtherComponents)
		{
			DebugPrint(ContainerColor, FString::Printf(TEXT("      ... %d more components not shown"), OtherCount - MaxOtherComponents));
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE
