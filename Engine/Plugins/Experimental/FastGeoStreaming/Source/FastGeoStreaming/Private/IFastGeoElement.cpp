// Copyright Epic Games, Inc. All Rights Reserved.

#include "IFastGeoElement.h"

uint32 FFastGeoElementType::NextUniqueID = 0;
const FFastGeoElementType IFastGeoElement::Type;

const FFastGeoElementType FFastGeoElementType::Invalid = FFastGeoElementType(FFastGeoElementType::EInitMode::Invalid);