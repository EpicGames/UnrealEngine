// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class FAutoConsoleVariableRef;
class UWorld;

class FFastGeoStreamingModule : public IModuleInterface
{
public:
	static FFastGeoStreamingModule& Get();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FASTGEOSTREAMING_API static bool IsFastGeoEnabled();
	FASTGEOSTREAMING_API static bool IsAsyncRenderWorkAllowed();

private:
	void OnWorldPreSendAllEndOfFrameUpdates(UWorld* InWorld);

private:
	FDelegateHandle Handle_OnWorldPreSendAllEndOfFrameUpdates;

	static bool bIsFastGeoEnabled;
	static FAutoConsoleVariableRef CVarIsEnabled;

#if !WITH_EDITOR
	static bool bAllowAsyncRenderWork;
	static FAutoConsoleVariableRef CVarAllowAsyncRenderWork;
#endif
};
