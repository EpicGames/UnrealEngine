// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"
#if WITH_EDITOR
#include "Templates/SubclassOf.h"
#include "HAL/IConsoleManager.h"
#endif 
#include "Engine/DataAsset.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.generated.h"

class UActorComponent;

enum class EFastGeoTransform : uint32
{
	Allow,		//!< Actor or component can be transformed.
	Reject,		//!< Actor or component can't be transformed.
	Discard,	//!< Actor or component is not relevant and can be fully discarded without impact on the game.
	MAX
};

inline constexpr uint32 EnumToIndex(EFastGeoTransform Value)
{
	return static_cast<uint32>(Value);
}

/** Stores the transformation result and optionally collects the failure reason for debug reports. */
struct FFastGeoTransformResult
{
public:
	FFastGeoTransformResult(EFastGeoTransform InTransformResult, const TCHAR* FailureReason = nullptr);
	FFastGeoTransformResult(EFastGeoTransform InTransformResult, TFunctionRef<FString()> FailureReasonFunc);

	EFastGeoTransform GetResult() const { return TransformResult; }
	uint32 GetResultIndex() const { return EnumToIndex(TransformResult); }
	const FString& GetReason() const { return Reason; }

	static bool bShouldCollectReasons;

private:
	EFastGeoTransform TransformResult;
	FString Reason;
};

/** Defines the memory storage format for FastGeo instance transforms. */
UENUM()
enum class EFastGeoInstanceStorageMode : uint8
{
	/** 
 	 * Automatically selects the storage format by checking if compression causes
 	 * precision loss in position, rotation, or scale beyond the allowed thresholds.
 	 */
	Auto,
	/** 
	 * Forces using FTransform3f (recommended for most game levels to optimize memory footprint).
	 */
	Compressed,
	/** 
	 * Forces using FMatrix (Best for absolute accuracy when instances show jitter or scale artifacts).
	 */
	Full
};

/** Threshold values used when transforming InstanceStaticMeshComponent tranforms when 
  * FastGeo transformer InstanceStorageMode is set to EFastGeoInstanceStorageMode::Auto.
  */
USTRUCT()
struct FASTGEOSTREAMING_API FFastGeoInstanceCompressionSettings
{
	GENERATED_BODY()

public:
	/** Threshold for Position (cm) */
	UPROPERTY(EditAnywhere, Category = "Instance Precision", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", Step = "0.001"))
	float PositionEpsilon = 5e-2f;

	/** Threshold for Rotation */
	UPROPERTY(EditAnywhere, Category = "Instance Precision", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "0.01", Step = "0.0001"))
	float RotationEpsilon = 1e-4f;

	/** Threshold for Scale */
	UPROPERTY(EditAnywhere, Category = "Instance Precision", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "0.01", Step = "0.0001"))
	float ScaleEpsilon = 1e-4f;
};

// Empty type used to present a Button in the property grid - not actual data
USTRUCT()
struct FFastGeoConversionButton
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories)
class UFastGeoTransformerSettings : public UDataAsset
{
	GENERATED_BODY()
	
public:
	//~ Begin UObject Interface.
	virtual void PostLoad() override;
	//~ End UObject Interface.

	/** Allowed actor classes (recursive) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta= (AllowAbstract = "true"))
	TArray<TSubclassOf<AActor>> AllowedActorClasses;

	/** Allowed actor classes (exact) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<AActor>> AllowedExactActorClasses;

	/** Allowed component classes (recursive) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> AllowedComponentClasses;

	/** Allowed component classes (exact) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> AllowedExactComponentClasses;

	/** Disallowed actor classes (recursive) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<AActor>> DisallowedActorClasses;

	/** Disallowed actor classes (exact) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<AActor>> DisallowedExactActorClasses;

	/** Disallowed component classes (recursive) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> DisallowedComponentClasses;

	/** Disallowed component classes (exact) to convert to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> DisallowedExactComponentClasses;

	/** Disallow any actor that contains a component of these classes (recursive) */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> DisallowedActorsContainingComponentClasses;

	/** Disallow any actor that contains a component of these exact classes */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> DisallowedActorsContainingExactComponentClasses;

	/** Component classes (recursive) to ignore when deciding to destroy a converted actor to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> IgnoredRemainingComponentClasses;

	/** Component classes (exact) to ignore when deciding to destroy a converted actor to FastGeo */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (AllowAbstract = "true"))
	TArray<TSubclassOf<UActorComponent>> IgnoredRemainingExactComponentClasses;

	/**
	  * Whether the transformer should generate a surrogate actor and surrogate components
	  * which will be linked to the physics body instances when initialized.
	  * When hitting FastGeo content, a surrogate component will be available in the HitResult & OverlapResult.
	  * HitResult.Item & OverlapResult.ItemIndex will map to the actual FastGeo component body instance index.
	  */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bGenerateSurrogateComponents = true;

	/** Determines how instance transforms are stored in memory */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EFastGeoInstanceStorageMode InstanceStorageMode = EFastGeoInstanceStorageMode::Auto;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "InstanceStorageMode == EFastGeoInstanceStorageMode::Auto", EditConditionHides))
	FFastGeoInstanceCompressionSettings InstanceCompressionSettings;

	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (DisplayName = "Convert settings to reusable Asset"))
	FFastGeoConversionButton FastGeoConversionButton;
};

UCLASS(Config = FastGeoStreaming, DefaultConfig)
class FASTGEOSTREAMING_API UFastGeoWorldPartitionRuntimeCellTransformer : public UWorldPartitionRuntimeCellTransformer
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

#if WITH_EDITOR
public:
	//~ Begin UWorldPartitionRuntimeCellTransformer Interface.
	virtual void Transform(ULevel* InLevel) override final;
	virtual void ForEachIgnoredComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const override final;
	virtual void ForEachIgnoredExactComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const override final;
	//~ End UWorldPartitionRuntimeCellTransformer Interface.

	//~ Begin UObject Interface.
	virtual void BeginDestroy() override;
	virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject Interface.

	/** Used internally to get transformer settings. */
	static const UFastGeoWorldPartitionRuntimeCellTransformer* GetCurrentTransformer() { return CurrentTransformer; }

protected:
	/** Whether the actor can be processed by the transformer. */
	virtual bool IsActorTransformable(AActor* InActor, FString& OutReason) const { return true; }
	/** Whether the component can be processed by the transformer. */
	virtual bool IsComponentTransformable(UActorComponent* InComponent, FString& OutReason) const { return true; }
	/** Whether the fully transformed actor can be deleted by the transformer. */
	virtual bool IsFullyTransformedActorDeletable(AActor* InActor, FString& OutReason) const { return true; }
	/** Whether component mobility is supported by the transformer. */
	bool IsComponentMobilitySupported(USceneComponent* InComponent, FString& OutReason) const;

	struct FTransformableActor
	{
		int32 ActorIndex = INDEX_NONE;
		bool bIsActorFullyTransformable = false;
		TArray<UActorComponent*> TransformableComponents;
	};

	struct FTransformationStats
	{
		int32 TotalActorCount = 0;
		int32 TotalComponentCount = 0;
		int32 FullyTransformableActorCount = 0;
		int32 PartiallyTransformableActorCount = 0;
		int32 TransformedComponentCount = 0;

		void DumpStats(const TCHAR* InPrefixString) const;
	};

	struct FComponentReport
	{
		UActorComponent* Component = nullptr;
		EFastGeoTransform Result = EFastGeoTransform::Allow;
		FString Reason;
	};

	struct FActorReport
	{
		AActor* Actor = nullptr;
		EFastGeoTransform Result = EFastGeoTransform::Allow;
		FString Reason;
		TArray<FComponentReport> ComponentReports;
	};

	FFastGeoTransformResult CanTransformActor(AActor* InActor, bool& bOutIsActorFullyTransformable, TArray<UActorComponent*>& OutTransformableComponents, TArray<FComponentReport>* OutComponentReports = nullptr) const;
	FFastGeoTransformResult IsAllowedActorClass(AActor* InActor) const;
	FFastGeoTransformResult CanTransformComponent(UActorComponent* InComponent) const;
	FFastGeoTransformResult IsAllowedComponentClass(UActorComponent* InComponent) const;
	bool CanRemoveActor(AActor* InActor, TSet<UActorComponent*>& InIgnoredComponents) const;
	bool CanAlwaysIgnoreActor(AActor* InActor) const;
	void OnSelectionChanged(UObject* Object);
	bool IsDebugMode() const;

	TMap<AActor*, TArray<AActor*>> BuildActorsReferencesMap(const TArray<AActor*>& InActors);
	void GatherTransformableActors(const TArray<AActor*>& InActors, const ULevel* InLevel, TArray<TPair<AActor*, FTransformableActor>>& OutTransformableActors, FTransformationStats& OutStats, TArray<FActorReport>* OutReports = nullptr);
	static void DumpDebugReport(const TCHAR* Title, const TArray<AActor*>& IgnoredActors, const TArray<FActorReport>& ActorReports, const FTransformationStats& Stats);
#endif

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UFastGeoTransformerSettings> Settings = nullptr;

	const UFastGeoTransformerSettings& GetSettings() const
	{
		check(EmbeddedSettings);
		return (Settings != nullptr) ? *Settings : *EmbeddedSettings;
	}
#endif

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Instanced, NoClear, Meta = (EditCondition = "Settings == nullptr", Tooltip = "Fallback only available when no Settings asset is used."), Category = "Settings")
	TObjectPtr<UFastGeoTransformerSettings> EmbeddedSettings;
	
	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category = "Debug")
	bool bDebugMode;

	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category = "Debug")
	bool bDebugSelectionMode;
	
	// The following properties are all deprecated in favor of the separate Settings property pointing to a reusable settings asset
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<AActor>> AllowedActorClasses_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<AActor>> AllowedExactActorClasses_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> AllowedComponentClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> AllowedExactComponentClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<AActor>> DisallowedActorClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<AActor>> DisallowedExactActorClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> DisallowedComponentClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> DisallowedExactComponentClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> IgnoredRemainingComponentClasses_DEPRECATED;
 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TSubclassOf<UActorComponent>> IgnoredRemainingExactComponentClasses_DEPRECATED;
#endif

#if WITH_EDITOR
	static bool IsDebugModeEnabled;
	static bool IsFastGeoTransformerEnabled;
	static FAutoConsoleVariableRef CVarIsDebugModeEnabled;
	static FAutoConsoleVariableRef CVarIsFastGeoTransformerEnabled;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Config)
	TArray<TSubclassOf<AActor>> BuiltinAllowedActorClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<AActor>> BuiltinDisallowedActorClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> BuiltinAllowedComponentClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> BuiltinDisallowedComponentClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> BuiltinIgnoredRemainingComponentClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> BuiltinIgnoredRemainingExactComponentClasses;
#endif

private:
	static UFastGeoWorldPartitionRuntimeCellTransformer* CurrentTransformer;
};