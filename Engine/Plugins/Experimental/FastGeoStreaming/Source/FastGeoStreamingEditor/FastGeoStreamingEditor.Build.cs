// Copyright Epic Games, Inc. All Rights Reserved.
namespace UnrealBuildTool.Rules
{
	public class FastGeoStreamingEditor : ModuleRules
	{
		public FastGeoStreamingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AssetTools",
					"Core",
					"CoreUObject",
					"Engine",
					"FastGeoStreaming",
					"Slate",
					"SlateCore",
					"UnrealEd"
				}
			);
		}
	}
}