// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoAssetDefinitions.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.h"

// This file is required so that UFastGeoTransformerSettings appears in Asset creation dialogs

#if WITH_EDITOR

FText UAssetDefinition_FastGeoTransformerSettings::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_FastGeoTransformerSettings", "FastGeo Transformer Settings");
}

FLinearColor UAssetDefinition_FastGeoTransformerSettings::GetAssetColor() const
{
	return FLinearColor(FColor(12, 65, 12));
}

TSoftClassPtr<UObject> UAssetDefinition_FastGeoTransformerSettings::GetAssetClass() const
{
	return UFastGeoTransformerSettings::StaticClass();
}

FAssetSupportResponse UAssetDefinition_FastGeoTransformerSettings::CanLocalize(const FAssetData& InAsset) const
{
	return FAssetSupportResponse::NotSupported();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_FastGeoTransformerSettings::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { EAssetCategoryPaths::World };
	return Categories;
}

#endif