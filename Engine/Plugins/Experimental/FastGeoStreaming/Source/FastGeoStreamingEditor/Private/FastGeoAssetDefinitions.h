// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "FastGeoAssetDefinitions.generated.h"

#if WITH_EDITOR

UCLASS()
class UAssetDefinition_FastGeoTransformerSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	// UAssetDefinition End
};

#endif