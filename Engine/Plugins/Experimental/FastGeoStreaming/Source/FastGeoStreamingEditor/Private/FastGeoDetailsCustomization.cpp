// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoDetailsCustomization.h"
#include "AssetToolsModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FastGeoFactory.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FastGeoDetailsCustomization"

TSharedRef<IPropertyTypeCustomization> FConvertFastGeoSettingsAssetButtonCustomization::MakeInstance()
{
	return MakeShareable(new FConvertFastGeoSettingsAssetButtonCustomization);
}

void FConvertFastGeoSettingsAssetButtonCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	bool bCreateConvertButton = false;

	// Only offer a conversion button when this special property is found on a UFastGeoTransformerSettings, which itself is owned directly by a UFastGeoWorldPartitionRuntimeCellTransformer
	// which means the UFastGeoTransformerSettings is the instanced EmbeddedSettings member, which we would like to make users convert to the separate Settings member
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	TWeakObjectPtr<UFastGeoTransformerSettings> TransformerSettings;
	TWeakObjectPtr<UFastGeoWorldPartitionRuntimeCellTransformer> Transformer;

	if (Objects.Num() == 1)
	{
		// Check that our special UFastGeoConversionButton property is indeed used in the UFastGeoTransformerSettings
		TransformerSettings = Cast<UFastGeoTransformerSettings>(Objects[0]);
		if (TransformerSettings != nullptr)
		{
			// And then check that we are not editing a standalone UFastGeoTransformerSettings asset, but rather the instanced EmbeddedSettings asset
			// (we could check for the name of the property if there are ever more than one instanced UFastGeoTransformerSettings)
			Transformer = Cast<UFastGeoWorldPartitionRuntimeCellTransformer>(TransformerSettings->GetOuter());
			if (Transformer != nullptr)
			{
				bCreateConvertButton = true;
			}
		}
	}

	if (!bCreateConvertButton)
	{
		// Skipping here means the whole property row is removed, which is what we want
		return;
	}

	HeaderRow.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([Transformer, TransformerSettings]()
					{
						if (!Transformer.IsValid() || !TransformerSettings.IsValid())
						{
							return FReply::Handled();
						}
						// Support undo
						const FScopedTransaction Transaction(LOCTEXT("ConvertFastGeoSettings", "Convert FastGeo Settings to standalone asset"));
						// Create a new settings asset, then populate it with the current settings, then install the new asset in the transformer to replace these embedded settings
						FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
						UFastGeoFactory* Factory = NewObject<UFastGeoFactory>();
						Factory->InitialSettings = TransformerSettings.Get(); // Set up the factory for a copy, since there are no AssetTools API to create an Asset with Dialog AND copy from an existing object
						UFastGeoTransformerSettings* SettingsAsset = Cast<UFastGeoTransformerSettings>(IAssetTools::Get().CreateAssetWithDialog(UFastGeoTransformerSettings::StaticClass(), Factory));
						if (SettingsAsset)
						{
							// Install the new Asset in the Transformer
							Transformer->Settings = SettingsAsset;
							Transformer->GetPackage()->SetDirtyFlag(true);
						}
						return FReply::Handled();
					})
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ConvertFastGeoSettingToAssetButton", "Convert to Asset"))
						.ToolTipText(LOCTEXT("ConvertFastGeoSettingToAsset_ToolTip", "Convert these settings to a separate reusable asset"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
}

#undef LOCTEXT_NAMESPACE