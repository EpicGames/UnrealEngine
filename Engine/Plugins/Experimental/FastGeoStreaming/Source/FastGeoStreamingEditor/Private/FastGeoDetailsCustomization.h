// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "IPropertyTypeCustomization.h"

// Unfortunately, because UFastGeoTransformerSettings are Instanced in the case where we want the convert button to be available, and because IDetailCustomization currently doesn't work for Instanced properties,
// we customize a virtual property instead
class FConvertFastGeoSettingsAssetButtonCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
};