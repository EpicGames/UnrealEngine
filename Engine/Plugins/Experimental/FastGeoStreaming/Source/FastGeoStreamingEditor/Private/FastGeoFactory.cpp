// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoFactory.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.h"

#if WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(FastGeoFactory)

UFastGeoFactory::UFastGeoFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFastGeoTransformerSettings::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UFastGeoFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (InitialSettings != nullptr)
	{
		return NewObject<UFastGeoTransformerSettings>(InParent, InName, Flags, InitialSettings);
	}
	
	return NewObject<UFastGeoTransformerSettings>(InParent, InName, Flags);
}

#endif