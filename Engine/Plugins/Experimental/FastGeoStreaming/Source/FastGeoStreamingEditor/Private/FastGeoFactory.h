// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "FastGeoFactory.generated.h"

#if WITH_EDITOR

UCLASS(hidecategories = Object, MinimalAPI)
class UFastGeoFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Set this in order for the factory to duplicate an object
	UPROPERTY()
	TObjectPtr<class UFastGeoTransformerSettings> InitialSettings;
	
	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End UFactory Interface	
};
#endif