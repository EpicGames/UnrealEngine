// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastGeoStreamingEditorModule.h"
#include "FastGeoDetailsCustomization.h"
#include "FastGeoWorldPartitionRuntimeCellTransformer.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "FastGeoComponentCluster.h"
#include "FastGeoContainer.h"
#include "FastGeoProceduralISMComponent.h"
#include "Selection.h"
#include "UObject/UObjectIterator.h"

IMPLEMENT_MODULE(FFastGeoStreamingEditorModule, FastGeoStreamingEditor);

void FFastGeoStreamingEditorModule::StartupModule()
{
	// Register property grid customization
	if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->RegisterCustomPropertyTypeLayout(FFastGeoConversionButton::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConvertFastGeoSettingsAssetButtonCustomization::MakeInstance));
	}

	USelection::SelectionChangedEvent.AddRaw(this, &FFastGeoStreamingEditorModule::OnEditorSelectionChanged);
}

void FFastGeoStreamingEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(FFastGeoConversionButton::StaticStruct()->GetFName());
		}
	}

	USelection::SelectionChangedEvent.RemoveAll(this);
}

void FFastGeoStreamingEditorModule::OnEditorSelectionChanged(UObject* /*ChangedObject*/)
{
	for (TObjectIterator<UFastGeoContainer> It(RF_ClassDefaultObject, /*bIncludeDerivedClasses=*/true, EInternalObjectFlags::Garbage); It; ++It)
	{
		It->ForEachComponentCluster([](FFastGeoComponentCluster& Cluster)
		{
			// Currently only Procedural ISM components support selection.
			Cluster.ForEachComponent<FFastGeoProceduralISMComponent>([](FFastGeoProceduralISMComponent& Component)
			{
				Component.UpdateEditorSelectionState();
			});
		});
	}
}
